[CoreRedirects]

;[4.25]
+ClassRedirects=(OldName="/Script/Synthesis.ConvolutionReverbPreset", NewName="/Script/Synthesis.SubmixEffectConvolutionReverbPreset")
+ClassRedirects=(OldName="/Script/Synthesis.ImpulseResponse", NewName="/Script/Synthesis.AudioImpulseResponse")

+PropertyRedirects=(OldName="/Script/Synthesis.ImpulseResponse.IRData", NewName="/Script/Synthesis.AudioImpulseResponse.IRData_DEPRECATED")
+PropertyRedirects=(OldName="/Script/Synthesis.AssetSampleRate", NewName="/Script/Synthesis.SampleRate")
