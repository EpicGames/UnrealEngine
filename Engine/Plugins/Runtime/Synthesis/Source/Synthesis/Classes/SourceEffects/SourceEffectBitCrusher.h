// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "DSP/BitCrusher.h"
#include "Sound/SoundEffectSource.h"
#include "Sound/SoundModulationDestination.h"

#include "SourceEffectBitCrusher.generated.h"

#define UE_API SYNTHESIS_API


// Forward Declarations
class USoundModulatorBase;

USTRUCT(BlueprintType)
struct FSourceEffectBitCrusherBaseSettings
{
	GENERATED_USTRUCT_BODY()

	// The reduced frequency to use for the audio stream. 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SourceEffect|Preset", meta = (ClampMin = "0.1", ClampMax = "96000.0", UIMin = "500.0", UIMax = "16000.0"))
	float SampleRate = 8000.0f;

	// The reduced bit depth to use for the audio stream
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SourceEffect|Preset", meta = (ClampMin = "1.0", ClampMax = "24.0", UIMin = "1.0", UIMax = "16.0"))
	float BitDepth = 8.0f;
};

USTRUCT(BlueprintType)
struct FSourceEffectBitCrusherSettings
{
	GENERATED_USTRUCT_BODY()

	// The reduced frequency to use for the audio stream. 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = BitCrusher, meta = (DisplayName = "Sample Rate", AudioParam = "SampleRate", AudioParamClass = "SoundModulationParameterFrequency", ClampMin = "0.1", ClampMax = "96000.0", UIMin = "500.0", UIMax = "16000.0"))
	FSoundModulationDestinationSettings SampleRateModulation;

	// The reduced bit depth to use for the audio stream
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = BitCrusher, meta = (DisplayName = "Bit Depth", AudioParam = "BitDepth", AudioParamClass = "SoundModulationParameterScaled", ClampMin = "1.0", ClampMax = "24.0", UIMin = "1.0", UIMax = "16.0"))
	FSoundModulationDestinationSettings BitModulation;

	FSourceEffectBitCrusherSettings()
	{
		SampleRateModulation.Value = 8000.0f;
		BitModulation.Value = 8.0f;
	}
};

class FSourceEffectBitCrusher : public FSoundEffectSource
{
public:
	// Called on an audio effect at initialization on main thread before audio processing begins.
	UE_API virtual void Init(const FSoundEffectSourceInitData& InitData) override;
	
	// Called when an audio effect preset is changed
	UE_API virtual void OnPresetChanged() override;

	// Process the input block of audio. Called on audio thread.
	UE_API virtual void ProcessAudio(const FSoundEffectSourceInputData& InData, float* OutAudioBufferData) override;

	UE_API void SetBitModulator(const USoundModulatorBase* Modulator);
	UE_API void SetBitModulators(const TSet<USoundModulatorBase*>& InModulators);

	UE_API void SetSampleRateModulator(const USoundModulatorBase* Modulator);
	UE_API void SetSampleRateModulators(const TSet<USoundModulatorBase*>& InModulators);

protected:
	Audio::FBitCrusher BitCrusher;

	FSourceEffectBitCrusherSettings SettingsCopy;

	Audio::FModulationDestination SampleRateMod;
	Audio::FModulationDestination BitMod;
};

UCLASS(MinimalAPI, ClassGroup = AudioSourceEffect, meta = (BlueprintSpawnableComponent))
class USourceEffectBitCrusherPreset : public USoundEffectSourcePreset
{
	GENERATED_BODY()

public:
	EFFECT_PRESET_METHODS(SourceEffectBitCrusher)

	UE_API virtual void OnInit() override;

	virtual FColor GetPresetColor() const override { return FColor(196.0f, 185.0f, 121.0f); }

	UFUNCTION(BlueprintCallable, Category = SourceEffectPreset)
	UE_API void SetBits(float Bits);

	UFUNCTION(BlueprintCallable, Category = SourceEffectPreset)
	UE_API void SetBitModulator(const USoundModulatorBase* Modulator);

	UFUNCTION(BlueprintCallable, Category = SourceEffectPreset)
	UE_API void SetBitModulators(const TSet<USoundModulatorBase*>& InModulators);

	UFUNCTION(BlueprintCallable, Category = SourceEffectPreset)
	UE_API void SetSampleRate(float SampleRate);

	UFUNCTION(BlueprintCallable, Category = SourceEffectPreset)
	UE_API void SetSampleRateModulator(const USoundModulatorBase* Modulator);

	UFUNCTION(BlueprintCallable, Category = SourceEffectPreset)
	UE_API void SetSampleRateModulators(const TSet<USoundModulatorBase*>& InModulators);

	// Sets just base (i.e. carrier) setting values without modifying modulation source references
	UFUNCTION(BlueprintCallable, Category = SourceEffectPreset)
	UE_API void SetSettings(const FSourceEffectBitCrusherBaseSettings& Settings);

	UFUNCTION(BlueprintCallable, Category = SourceEffectPreset)
	UE_API void SetModulationSettings(const FSourceEffectBitCrusherSettings& ModulationSettings);

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = SourceEffectPreset, meta = (ShowOnlyInnerProperties))
	FSourceEffectBitCrusherSettings Settings;
};

#undef UE_API
