// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DSP/Chorus.h"
#include "Sound/SoundEffectSource.h"
#include "Sound/SoundModulationDestination.h"

#include "SourceEffectChorus.generated.h"

#define UE_API SYNTHESIS_API

USTRUCT(BlueprintType)
struct FSourceEffectChorusBaseSettings
{
	GENERATED_USTRUCT_BODY()

	// The depth of the chorus effect
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SourceEffect|Preset", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float Depth = 0.2f;

	// The frequency of the chorus effect
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SourceEffect|Preset", meta = (UIMin = "0.0", UIMax = "5.0"))
	float Frequency = 2.0f;

	// The feedback of the chorus effect
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SourceEffect|Preset", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float Feedback = 0.3f;

	// The wet level of the chorus effect
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SourceEffect|Preset", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float WetLevel = 0.5f;

	// The dry level of the chorus effect
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SourceEffect|Preset", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float DryLevel = 0.5f;

	// The spread of the effect (larger means greater difference between left and right delay lines)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SourceEffect|Preset", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float Spread = 0.0f;
};

USTRUCT(BlueprintType)
struct FSourceEffectChorusSettings
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(meta = (DeprecatedProperty))
	float Depth = 0.2f;

	UPROPERTY(meta = (DeprecatedProperty))
	float Frequency = 2.0f;

	UPROPERTY(meta = (DeprecatedProperty))
	float Feedback = 0.3f;

	UPROPERTY(meta = (DeprecatedProperty))
	float WetLevel = 0.5f;

	UPROPERTY(meta = (DeprecatedProperty))
	float DryLevel = 0.5f;

	UPROPERTY(meta = (DeprecatedProperty))
	float Spread = 0.0f;

	// The depth of the chorus effect
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Chorus, meta = (DisplayName = "Depth", AudioParamClass = "SoundModulationParameter", ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	FSoundModulationDestinationSettings DepthModulation;

	// The frequency of the chorus effect
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Chorus, meta = (DisplayName = "Frequency (hz)", AudioParam = "LowRateFrequency", AudioParamClass = "SoundModulationParameterFrequency", UIMin = "0.0", UIMax = "5.0"))
	FSoundModulationDestinationSettings FrequencyModulation;

	// The feedback of the chorus effect
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Chorus, meta = (DisplayName = "Feedback", AudioParamClass = "SoundModulationParameter", ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	FSoundModulationDestinationSettings FeedbackModulation;

	// The wet level of the chorus effect
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Chorus, meta = (DisplayName = "Wet Level", AudioParamClass = "SoundModulationParameterVolume", ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	FSoundModulationDestinationSettings WetModulation;

	// The dry level of the chorus effect
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Chorus, meta = (DisplayName = "Dry Level", AudioParamClass = "SoundModulationParameterVolume", ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	FSoundModulationDestinationSettings DryModulation;

	// The spread of the effect (larger means greater difference between left and right delay lines)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Chorus, meta = (DisplayName = "Spread", AudioParamClass = "SoundModulationParameter", ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	FSoundModulationDestinationSettings SpreadModulation;

	FSourceEffectChorusSettings()
	{
		DepthModulation.Value = 0.2f;
		FrequencyModulation.Value = 2.0f;
		FeedbackModulation.Value = 0.3f;
		WetModulation.Value = 0.5f;
		DryModulation.Value = 0.5f;
		DepthModulation.Value = 0.0f;
	}
};

class FSourceEffectChorus : public FSoundEffectSource
{
public:
	// Called on an audio effect at initialization on main thread before audio processing begins.
	UE_API virtual void Init(const FSoundEffectSourceInitData& InitData) override;
	
	// Called when an audio effect preset is changed
	UE_API virtual void OnPresetChanged() override;

	// Process the input block of audio. Called on audio thread.
	UE_API virtual void ProcessAudio(const FSoundEffectSourceInputData& InData, float* OutAudioBufferData) override;

	UE_API void SetDepthModulator(const USoundModulatorBase* InModulator);
	UE_API void SetFeedbackModulator(const USoundModulatorBase* InModulator);
	UE_API void SetFrequencyModulator(const USoundModulatorBase* InModulator);
	UE_API void SetWetModulator(const USoundModulatorBase* InModulator);
	UE_API void SetDryModulator(const USoundModulatorBase* InModulator);
	UE_API void SetSpreadModulator(const USoundModulatorBase* InModulator);

	UE_API void SetDepthModulators(const TSet<USoundModulatorBase*>& InModulators);
	UE_API void SetFeedbackModulators(const TSet<USoundModulatorBase*>& InModulators);
	UE_API void SetSpreadModulators(const TSet<USoundModulatorBase*>& InModulators);
	UE_API void SetDryModulators(const TSet<USoundModulatorBase*>& InModulators);
	UE_API void SetWetModulators(const TSet<USoundModulatorBase*>& InModulators);
	UE_API void SetFrequencyModulators(const TSet<USoundModulatorBase*>& InModulators);

protected:
	Audio::FChorus Chorus;

	FSourceEffectChorusSettings SettingsCopy;

	Audio::FModulationDestination DepthMod;
	Audio::FModulationDestination FeedbackMod;
	Audio::FModulationDestination FrequencyMod;
	Audio::FModulationDestination WetMod;
	Audio::FModulationDestination DryMod;
	Audio::FModulationDestination SpreadMod;
};

UCLASS(MinimalAPI, ClassGroup = AudioSourceEffect, meta = (BlueprintSpawnableComponent))
class USourceEffectChorusPreset : public USoundEffectSourcePreset
{
	GENERATED_BODY()

public:
	EFFECT_PRESET_METHODS(SourceEffectChorus)

	virtual FColor GetPresetColor() const override { return FColor(102.0f, 85.0f, 121.0f); }

	UE_API virtual void OnInit() override;

	UFUNCTION(BlueprintCallable, Category = "Audio|Effects|Chorus")
	UE_API void SetDepth(float Depth);

	UFUNCTION(BlueprintCallable, Category = "Audio|Effects|Chorus")
	UE_API void SetDepthModulator(const USoundModulatorBase* Modulator);

	UFUNCTION(BlueprintCallable, Category = "Audio|Effects|Chorus")
	UE_API void SetDepthModulators(const TSet<USoundModulatorBase*>& Modulators);

	UFUNCTION(BlueprintCallable, Category = "Audio|Effects|Chorus")
	UE_API void SetFeedback(float Feedback);

	UFUNCTION(BlueprintCallable, Category = "Audio|Effects|Chorus")
	UE_API void SetFeedbackModulator(const USoundModulatorBase* Modulator);

	UFUNCTION(BlueprintCallable, Category = "Audio|Effects|Chorus")
	UE_API void SetFeedbackModulators(const TSet<USoundModulatorBase*>& Modulators);

	UFUNCTION(BlueprintCallable, Category = "Audio|Effects|Chorus")
	UE_API void SetFrequency(float Frequency);

	UFUNCTION(BlueprintCallable, Category = "Audio|Effects|Chorus")
	UE_API void SetFrequencyModulator(const USoundModulatorBase* Modulator);

	UFUNCTION(BlueprintCallable, Category = "Audio|Effects|Chorus")
	UE_API void SetFrequencyModulators(const TSet<USoundModulatorBase*>& Modulators);

	UFUNCTION(BlueprintCallable, Category = "Audio|Effects|Chorus")
	UE_API void SetWet(float WetAmount);

	UFUNCTION(BlueprintCallable, Category = "Audio|Effects|Chorus")
	UE_API void SetWetModulator(const USoundModulatorBase* Modulator);

	UFUNCTION(BlueprintCallable, Category = "Audio|Effects|Chorus")
	UE_API void SetWetModulators(const TSet<USoundModulatorBase*>& Modulators);

	UFUNCTION(BlueprintCallable, Category = "Audio|Effects|Chorus")
	UE_API void SetDry(float DryAmount);

	UFUNCTION(BlueprintCallable, Category = "Audio|Effects|Chorus")
	UE_API void SetDryModulator(const USoundModulatorBase* Modulator);

	UFUNCTION(BlueprintCallable, Category = "Audio|Effects|Chorus")
	UE_API void SetDryModulators(const TSet<USoundModulatorBase*>& Modulators);

	UFUNCTION(BlueprintCallable, Category = "Audio|Effects|Chorus")
	UE_API void SetSpread(float Spread);

	UFUNCTION(BlueprintCallable, Category = "Audio|Effects|Chorus")
	UE_API void SetSpreadModulator(const USoundModulatorBase* Modulator);

	UFUNCTION(BlueprintCallable, Category = "Audio|Effects|Chorus")
	UE_API void SetSpreadModulators(const TSet<USoundModulatorBase*>& Modulators);

	// Sets just base (i.e. carrier) setting values without modifying modulation source references
	UFUNCTION(BlueprintCallable, Category = "Audio|Effects|Chorus")
	UE_API void SetSettings(const FSourceEffectChorusBaseSettings& Settings);

	UFUNCTION(BlueprintCallable, Category = "Audio|Effects|Chorus")
	UE_API void SetModulationSettings(const FSourceEffectChorusSettings& ModulationSettings);

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = SourceEffectPreset, meta = (ShowOnlyInnerProperties))
	FSourceEffectChorusSettings Settings;
};

#undef UE_API
