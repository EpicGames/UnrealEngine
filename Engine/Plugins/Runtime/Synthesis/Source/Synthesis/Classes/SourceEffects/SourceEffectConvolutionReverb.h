// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Sound/SoundEffectSource.h"
#include "SourceEffectConvolutionReverb.generated.h"

#define UE_API SYNTHESIS_API

class UAudioImpulseResponse;
class USourceEffectConvolutionReverbPreset;
enum class ESubmixEffectConvolutionReverbBlockSize : uint8;
namespace Audio { class FEffectConvolutionReverb; }
namespace AudioConvReverbIntrinsics { struct FVersionData; }

USTRUCT(BlueprintType)
struct FSourceEffectConvolutionReverbSettings
{
	GENERATED_USTRUCT_BODY()

	UE_API FSourceEffectConvolutionReverbSettings();

	/* Used to account for energy added by convolution with "loud" Impulse Responses. 
	 * This value is not directly editable in the editor because it is copied from the 
	 * associated UAudioImpulseResponse. */
	UPROPERTY();
	float NormalizationVolumeDb;

	// Controls how much of the wet signal is mixed into the output, in Decibels
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = ConvolutionReverb, meta = (DisplayName = "Wet Volume (dB)", ClampMin = "-96.0", ClampMax = "0.0", EditCondition = "!bBypass"));
	float WetVolumeDb;

	// Controls how much of the dry signal is mixed into the output, in Decibels
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = ConvolutionReverb, meta = (DisplayName = "Dry Volume (dB)", ClampMin = "-96.0", ClampMax = "0.0", EditCondition = "!bBypass"));
	float DryVolumeDb;

	/* If true, input audio is directly routed to output audio with applying any effect. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = ConvolutionReverb)
	bool bBypass;
};


/** Audio render thread effect object. */
class FSourceEffectConvolutionReverb : public FSoundEffectSource
{
public:
	FSourceEffectConvolutionReverb() = delete;
	// Construct a convolution object with an existing preset. 
	UE_API FSourceEffectConvolutionReverb(const USourceEffectConvolutionReverbPreset* InPreset);

	UE_API ~FSourceEffectConvolutionReverb();

	// Called on an audio effect at initialization on main thread before audio processing begins.
	UE_API virtual void Init(const FSoundEffectSourceInitData& InInitData) override;

	// Called when an audio effect preset settings is changed.
	UE_API virtual void OnPresetChanged() override;

	// Process the input block of audio. Called on audio thread.
	UE_API virtual void ProcessAudio(const FSoundEffectSourceInputData& InData, float* OutAudioBufferData) override;

	// Call on the game thread in order to update the impulse response and hardware acceleration
	// used in this effect.
	UE_API AudioConvReverbIntrinsics::FVersionData UpdateConvolutionReverb(const USourceEffectConvolutionReverbPreset* InPreset);

	UE_API void RebuildConvolutionReverb();
private:
	// Sets current runtime settings for convolution reverb which do *not* trigger
	// a FConvolutionReverb rebuild.  These settings will be applied to FConvolutionReverb 
	// at the next call to UpdateParameters()
	UE_API void SetConvolutionReverbParameters(const FSourceEffectConvolutionReverbSettings& InSettings);

	// Reverb performs majority of DSP operations
	TSharedRef<Audio::FEffectConvolutionReverb> Reverb;

	float WetVolume = 1.f;
	float DryVolume = 1.f;
	int32 NumChannels = 0;
};

UCLASS(MinimalAPI)
class USourceEffectConvolutionReverbPreset : public USoundEffectSourcePreset
{
	GENERATED_BODY()

public:
	
	UE_API USourceEffectConvolutionReverbPreset(const FObjectInitializer& ObjectInitializer);

	/* Note: The SourceEffect boilerplate macros could not be utilized here because
	 * the "CreateNewEffect" implementation differed from those available in the
	 * boilerplate macro.
	 */

	UE_API virtual bool CanFilter() const override;

	UE_API virtual bool HasAssetActions() const;

	UE_API virtual FText GetAssetActionName() const override;

	UE_API virtual UClass* GetSupportedClass() const override;

	UE_API virtual FSoundEffectBase* CreateNewEffect() const override;

	UE_API virtual USoundEffectPreset* CreateNewPreset(UObject* InParent, FName Name, EObjectFlags Flags) const override;

	UE_API virtual void Init() override;


	UE_API FSourceEffectConvolutionReverbSettings GetSettings() const;

	/** Set the convolution reverb settings */
	UFUNCTION(BlueprintCallable, Category = "Audio|Effects")
	UE_API void SetSettings(const FSourceEffectConvolutionReverbSettings& InSettings);

	/** Set the convolution reverb impulse response */
	UFUNCTION(BlueprintCallable, BlueprintSetter, Category = "Audio|Effects")
	UE_API void SetImpulseResponse(UAudioImpulseResponse* InImpulseResponse);

	/** The impulse response used for convolution. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter = SetImpulseResponse, Category = SourceEffectPreset)
	TObjectPtr<UAudioImpulseResponse> ImpulseResponse;

	/** ConvolutionReverbPreset Preset Settings. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter = SetSettings, Category = SourceEffectPreset, meta = (ShowOnlyInnerProperties))
	FSourceEffectConvolutionReverbSettings Settings;


	/** Set the internal block size. This can effect latency and performance. Higher values will result in
	 * lower CPU costs while lower values will result higher CPU costs. Latency may be affected depending
	 * on the interplay between audio engines buffer sizes and this effects block size. Generally, higher
	 * values result in higher latency, and lower values result in lower latency. 
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, AdvancedDisplay, Category = SourceEffectPreset)
	ESubmixEffectConvolutionReverbBlockSize BlockSize;

	/** Opt into hardware acceleration of the convolution reverb (if available) */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, AdvancedDisplay, Category = SourceEffectPreset)
	bool bEnableHardwareAcceleration;
	

#if WITH_EDITORONLY_DATA
	// Binds to the UAudioImpulseRespont::OnObjectPropertyChanged delegate of the current ImpulseResponse
	UE_API void BindToImpulseResponseObjectChange();

	UE_API virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	// Called when a property changes on the ImpulseResponse object
	UE_API void PostEditChangeImpulseProperty(FPropertyChangedEvent& PropertyChangedEvent);
#endif

	UE_API virtual void PostLoad() override;

private:
	void SetImpulseResponseSettings(UAudioImpulseResponse* InImpulseResponse);

	void UpdateSettings();

	void UpdateDeprecatedProperties();

	// This method requires that the effect is registered with a preset.  If this 
	// effect is not registered with a preset, then this will not update the convolution
	// algorithm.
	void RebuildConvolutionReverb();

	mutable FCriticalSection SettingsCritSect; 
	FSourceEffectConvolutionReverbSettings SettingsCopy; 

#if WITH_EDITORONLY_DATA

	TMap<UObject*, FDelegateHandle> DelegateHandles;
#endif
};

#undef UE_API
