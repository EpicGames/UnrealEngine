// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Sound/SoundEffectSource.h"
#include "DSP/DynamicsProcessor.h"
#include "SourceEffectDynamicsProcessor.generated.h"

#define UE_API SYNTHESIS_API

UENUM(BlueprintType)
enum class ESourceEffectDynamicsProcessorType : uint8
{
	Compressor = 0,
	Limiter,
	Expander,
	Gate,
	UpwardsCompressor,
	Count UMETA(Hidden)
};

UENUM(BlueprintType)
enum class ESourceEffectDynamicsPeakMode : uint8
{
	MeanSquared = 0,
	RootMeanSquared,
	Peak,
	Count UMETA(Hidden)
};

USTRUCT(BlueprintType)
struct FSourceEffectDynamicsProcessorSettings
{
	GENERATED_USTRUCT_BODY()

	// Type of processor to apply
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = General, meta = (DisplayName = "Type"))
	ESourceEffectDynamicsProcessorType DynamicsProcessorType = ESourceEffectDynamicsProcessorType::Compressor;

	// Mode of peak detection used on input key signal
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Dynamics, meta = (EditCondition = "!bBypass"))
	ESourceEffectDynamicsPeakMode PeakMode = ESourceEffectDynamicsPeakMode::RootMeanSquared;

	// The amount of time to look ahead of the current audio (Allows for transients to be included in dynamics processing)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Response,  meta = (DisplayName = "Look Ahead (ms)", ClampMin = "0.0", ClampMax = "50.0", UIMin = "0.0", UIMax = "50.0", EditCondition = "!bBypass"))
	float LookAheadMsec = 3.0f;

	// The amount of time to ramp into any dynamics processing effect
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Response, meta = (DisplayName = "Attack Time (ms)", ClampMin = "1.0", ClampMax = "300.0", UIMin = "1.0", UIMax = "200.0", EditCondition = "!bBypass"))
	float AttackTimeMsec = 10.0f;

	// The amount of time to release the dynamics processing effect
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Response, meta = (DisplayName = "Release Time (ms)", ClampMin = "20.0", ClampMax = "5000.0", UIMin = "20.0", UIMax = "5000.0", EditCondition = "!bBypass"))
	float ReleaseTimeMsec = 100.0f;

	// The threshold at which to perform a dynamics processing operation
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Dynamics, meta = (DisplayName = "Threshold (dB)", ClampMin = "-60.0", ClampMax = "0.0", UIMin = "-60.0", UIMax = "0.0", EditCondition = "!bBypass"))
	float ThresholdDb = -6.0f;

	// The dynamics processor ratio used for compression/expansion
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Dynamics, meta = (ClampMin = "1.0", ClampMax = "20.0", UIMin = "1.0", UIMax = "20.0", EditCondition = "!bBypass"))
	float Ratio = 1.5f;

	// The knee bandwidth of the processor to use
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Dynamics, meta = (DisplayName = "Knee Bandwidth (dB)", ClampMin = "0.0", ClampMax = "20.0", UIMin = "0.0", UIMax = "20.0", EditCondition = "!bBypass"))
	float KneeBandwidthDb = 10.0f;

	// The input gain of the dynamics processor
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = General, meta = (DisplayName = "Input Gain (dB)", ClampMin = "-12.0", ClampMax = "20.0", UIMin = "-12.0", UIMax = "20.0", EditCondition = "!bBypass"))
	float InputGainDb = 0.0f;

	// The output gain of the dynamics processor
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = General, meta = (DisplayName = "Output Gain (dB)", ClampMin = "0.0", ClampMax = "20.0", UIMin = "0.0", UIMax = "20.0", EditCondition = "!bBypass"))
	float OutputGainDb = 0.0f;

	// Whether the left and right channels are linked when determining envelopes
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Dynamics, meta = (EditCondition = "!bBypass"))
	uint32 bStereoLinked : 1;

	// Toggles treating the attack and release envelopes as analog-style vs digital-style (Analog will respond a bit more naturally/slower)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Response, meta = (EditCondition = "!bBypass"))
	uint32 bAnalogMode : 1;

	// Whether or not to bypass effect
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = General, meta = (DisplayName = "Bypass", DisplayAfter = "DynamicsProcessorType"))
	uint8 bBypass : 1;

	FSourceEffectDynamicsProcessorSettings()
		: bStereoLinked(true)
		, bAnalogMode(true)
		, bBypass(false)
	{
	}
};

class FSourceEffectDynamicsProcessor : public FSoundEffectSource
{
public:
	UE_API FSourceEffectDynamicsProcessor();

	// Called on an audio effect at initialization on main thread before audio processing begins.
	UE_API virtual void Init(const FSoundEffectSourceInitData& InitData) override;
	
	// Called when an audio effect preset is changed
	UE_API virtual void OnPresetChanged() override;

	// Process the input block of audio. Called on audio thread.
	UE_API virtual void ProcessAudio(const FSoundEffectSourceInputData& InData, float* OutAudioBufferData) override;

protected:

	Audio::FDynamicsProcessor DynamicsProcessor;
	bool bBypass = false;
};



UCLASS(MinimalAPI, ClassGroup = AudioSourceEffect, meta = (BlueprintSpawnableComponent))
class USourceEffectDynamicsProcessorPreset : public USoundEffectSourcePreset
{
	GENERATED_BODY()

public:
	EFFECT_PRESET_METHODS(SourceEffectDynamicsProcessor)

	virtual FColor GetPresetColor() const override { return FColor(218.0f, 199.0f, 11.0f); }

	UFUNCTION(BlueprintCallable, Category = SourceEffectPresets)
	UE_API void SetSettings(const FSourceEffectDynamicsProcessorSettings& InSettings);

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = SourceEffectPreset, meta = (ShowOnlyInnerProperties))
	FSourceEffectDynamicsProcessorSettings Settings;
};

#undef UE_API
