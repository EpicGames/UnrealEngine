// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Sound/SoundEffectSource.h"
#include "SourceEffectEQ.generated.h"

#define UE_API SYNTHESIS_API

namespace Audio { class FBiquadFilter; }


USTRUCT(BlueprintType)
struct FSourceEffectEQBand
{
	GENERATED_USTRUCT_BODY()

	// The cutoff frequency of the band
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Equalizer, meta = (DisplayName = "Frequency (hz)", ClampMin = "20.0", ClampMax = "20000.0", UIMin = "0.0", UIMax = "15000.0", EditCondition = "bEnabled"))
	float Frequency;

	// The bandwidth (in octaves) of the band
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Equalizer, meta = (DisplayName = "Bandwidth (octaves)", ClampMin = "0.1", ClampMax = "2.0", UIMin = "0.1", UIMax = "2.0", EditCondition = "bEnabled"))
	float Bandwidth;

	// The gain in decibels to apply to the eq band
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Equalizer, meta = (DisplayName = "Gain (dB)", ClampMin = "-90.0", ClampMax = "20.0", UIMin = "-90.0", UIMax = "20.0", EditCondition = "bEnabled"))
	float GainDb;

	// Whether or not the band is enabled. Allows changing bands on the fly.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Equalizer, meta = (DisplayName = "Enabled"))
	uint32 bEnabled : 1;

	FSourceEffectEQBand()
		: Frequency(500.0f)
		, Bandwidth(2.0f)
		, GainDb(0.0f)
		, bEnabled(false)
	{
	}
};

// EQ source effect settings
USTRUCT(BlueprintType)
struct FSourceEffectEQSettings
{
	GENERATED_USTRUCT_BODY()

	// The EQ bands to use
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SourceEffectPreset)
	TArray<FSourceEffectEQBand> EQBands;
};

class FSourceEffectEQ : public FSoundEffectSource
{
public:
	SYNTHESIS_API FSourceEffectEQ();

	// Called on an audio effect at initialization on main thread before audio processing begins.
	SYNTHESIS_API virtual void Init(const FSoundEffectSourceInitData& InitData) override;
	
	// Called when an audio effect preset is changed
	SYNTHESIS_API virtual void OnPresetChanged() override;

	// Process the input block of audio. Called on audio thread.
	SYNTHESIS_API virtual void ProcessAudio(const FSoundEffectSourceInputData& InData, float* OutAudioBufferData) override;

protected:

	// Bank of biquad filters
	TArray<Audio::FBiquadFilter> Filters;
	float InAudioFrame[2];
	float OutAudioFrame[2];
	float SampleRate;
	int32 NumChannels;
};

UCLASS(MinimalAPI, ClassGroup = AudioSourceEffect, meta = (BlueprintSpawnableComponent))
class USourceEffectEQPreset : public USoundEffectSourcePreset
{
	GENERATED_BODY()

public:
	EFFECT_PRESET_METHODS(SourceEffectEQ)

	virtual FColor GetPresetColor() const override { return FColor(53.0f, 158.0f, 153.0f); }

	UFUNCTION(BlueprintCallable, Category = "Audio|Effects")
	UE_API void SetSettings(const FSourceEffectEQSettings& InSettings);

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = SourceEffectPreset, meta = (ShowOnlyInnerProperties))
	FSourceEffectEQSettings Settings;
};

#undef UE_API
