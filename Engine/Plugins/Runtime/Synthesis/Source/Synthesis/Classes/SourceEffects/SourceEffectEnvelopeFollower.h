// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Sound/SoundEffectSource.h"
#include "DSP/EnvelopeFollower.h"
#include "Components/ActorComponent.h"
#include "SourceEffectEnvelopeFollower.generated.h"

#define UE_API SYNTHESIS_API

class UEnvelopeFollowerListener;

UENUM(BlueprintType)
enum class EEnvelopeFollowerPeakMode : uint8
{
	MeanSquared = 0,
	RootMeanSquared,
	Peak,
	Count UMETA(Hidden)
};

USTRUCT(BlueprintType)
struct FSourceEffectEnvelopeFollowerSettings
{
	GENERATED_USTRUCT_BODY()

	// The attack time of the envelope follower in milliseconds
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = EnvelopeFollower, meta = (DisplayName = "Attack Time (ms)", ClampMin = "0.0", UIMin = "0.0"))
	float AttackTime;

	// The release time of the envelope follower in milliseconds
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = EnvelopeFollower, meta = (DisplayName = "Release Time (ms)", ClampMin = "0.0", UIMin = "0.0"))
	float ReleaseTime;

	// The peak mode of the envelope follower
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = EnvelopeFollower)
	EEnvelopeFollowerPeakMode PeakMode;

	// Whether or not the envelope follower is in analog mode
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = EnvelopeFollower)
	bool bIsAnalogMode;

	FSourceEffectEnvelopeFollowerSettings()
		: AttackTime(10.0f)
		, ReleaseTime(100.0f)
		, PeakMode(EEnvelopeFollowerPeakMode::Peak)
		, bIsAnalogMode(true)
	{}
};

class FSourceEffectEnvelopeFollower : public FSoundEffectSource
{
public:
	UE_API ~FSourceEffectEnvelopeFollower();

	// Called on an audio effect at initialization on main thread before audio processing begins.
	UE_API virtual void Init(const FSoundEffectSourceInitData& InitData) override;
	
	// Called when an audio effect preset is changed
	UE_API virtual void OnPresetChanged() override;

	// Process the input block of audio. Called on audio thread.
	UE_API virtual void ProcessAudio(const FSoundEffectSourceInputData& InData, float* OutAudioBufferData) override;

protected:

	Audio::FInlineEnvelopeFollower EnvelopeFollower;
	float CurrentEnvelopeValue;
	uint32 OwningPresetUniqueId;
	uint32 InstanceId;
	int32 FrameCount;
	int32 FramesToNotify;
	int32 NumChannels;
};

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnEnvelopeFollowerUpdate, float, EnvelopeValue);

class IEnvelopeFollowerNotifier
{
public:
	virtual void UnregisterEnvelopeFollowerListener(uint32 PresetUniqueId, UEnvelopeFollowerListener* EnvFollowerListener) = 0;
};

UCLASS(MinimalAPI, ClassGroup = Synth, hidecategories = (Object, ActorComponent, Physics, Rendering, Mobility, LOD), meta = (BlueprintSpawnableComponent))
class UEnvelopeFollowerListener : public UActorComponent
{
	GENERATED_BODY()

public:
	UEnvelopeFollowerListener(const FObjectInitializer& ObjInit)
		: Super(ObjInit)
		, bRegistered(false)
		, PresetUniqueId(INDEX_NONE)
		, EnvelopeFollowerNotifier(nullptr)
	{
	}

	~UEnvelopeFollowerListener()
	{
		if (EnvelopeFollowerNotifier)
		{
			check(PresetUniqueId != INDEX_NONE);
			EnvelopeFollowerNotifier->UnregisterEnvelopeFollowerListener(PresetUniqueId, this);
		}
	}

	UPROPERTY(BlueprintAssignable)
	FOnEnvelopeFollowerUpdate OnEnvelopeFollowerUpdate;

	void Init(IEnvelopeFollowerNotifier* InNotifier, uint32 InPresetUniqueId)
	{
		if (PresetUniqueId != INDEX_NONE)
		{
			check(InNotifier);
			InNotifier->UnregisterEnvelopeFollowerListener(PresetUniqueId, this);
		}
		PresetUniqueId = InPresetUniqueId;
		EnvelopeFollowerNotifier = InNotifier;
	}

protected:
	bool bRegistered;
	uint32 PresetUniqueId;
	IEnvelopeFollowerNotifier* EnvelopeFollowerNotifier;
};

UCLASS(MinimalAPI, ClassGroup = AudioSourceEffect, meta = (BlueprintSpawnableComponent))
class USourceEffectEnvelopeFollowerPreset : public USoundEffectSourcePreset
{
	GENERATED_BODY()

public:
	EFFECT_PRESET_METHODS(SourceEffectEnvelopeFollower)

	virtual FColor GetPresetColor() const override { return FColor(248.0f, 218.0f, 78.0f); }

	UFUNCTION(BlueprintCallable, Category = "Audio|Effects")
	UE_API void SetSettings(const FSourceEffectEnvelopeFollowerSettings& InSettings);

	/** Registers an envelope follower listener with the effect. */
	UFUNCTION(BlueprintCallable, Category = "Audio|Effects", meta = (WorldContext = "WorldContextObject"))
	UE_API void RegisterEnvelopeFollowerListener(UEnvelopeFollowerListener* EnvelopeFollowerListener);

	/** Unregisters an envelope follower listener with the effect. */
	UFUNCTION(BlueprintCallable, Category = "Audio|Effects", meta = (WorldContext = "WorldContextObject"))
	UE_API void UnregisterEnvelopeFollowerListener(UEnvelopeFollowerListener* EnvelopeFollowerListener);

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = SourceEffectPreset, meta = (ShowOnlyInnerProperties))
	FSourceEffectEnvelopeFollowerSettings Settings;
};

#undef UE_API
