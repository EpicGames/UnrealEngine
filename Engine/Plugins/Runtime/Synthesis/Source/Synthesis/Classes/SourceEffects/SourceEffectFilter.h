// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Sound/SoundEffectSource.h"
#include "DSP/Filter.h"
#include "DSP/MultithreadedPatching.h"
#include "DSP/EnvelopeFollower.h"
#include "SourceEffectFilter.generated.h"

#define UE_API SYNTHESIS_API

class UAudioBus;

UENUM(BlueprintType)
enum class ESourceEffectFilterCircuit : uint8
{
	OnePole = 0,
	StateVariable,
	Ladder,
	Count UMETA(Hidden)
};

UENUM(BlueprintType)
enum class ESourceEffectFilterType : uint8
{
	LowPass = 0,
	HighPass,
	BandPass,
	BandStop,
	Count UMETA(Hidden)
};

UENUM(BlueprintType)
enum class ESourceEffectFilterParam : uint8
{
	FilterFrequency = 0,
	FilterResonance,
	Count UMETA(Hidden)
};

USTRUCT(BlueprintType)
struct FSourceEffectFilterAudioBusModulationSettings
{
	GENERATED_USTRUCT_BODY()

	// Audio bus to use to modulate the filter
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Filter)
	TObjectPtr<UAudioBus> AudioBus = nullptr;

	// The amplitude envelope follower attack time (in milliseconds) on the audio bus.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Filter, meta = (ClampMin = "0", UIMin = "0", UIMax = "2000.0", DisplayName = "Attack Time (ms)"))
	int32 EnvelopeFollowerAttackTimeMsec = 10;

	// The amplitude envelope follower release time (in milliseconds) on the audio bus.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Filter, meta = (ClampMin = "0", UIMin = "0", UIMax = "2000.0", DisplayName = "Release Time (ms)"))
	int32 EnvelopeFollowerReleaseTimeMsec = 100;

	// An amount to scale the envelope follower output to map to the modulation values.  
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Filter, meta = (ClampMin = "0"))
	float EnvelopeGainMultiplier = 1.0f;

	// Which parameter to modulate.
	UPROPERTY(EditAnywhere, Category = ModulationParameters, meta = (InlineCategoryProperty))
	ESourceEffectFilterParam FilterParam = ESourceEffectFilterParam::FilterFrequency;

	// The frequency modulation value (in semitones from the filter frequency) to use when the envelope is quietest
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = ModulationParameters, meta = (EditCondition = "FilterParam == ESourceEffectFilterParam::FilterFrequency", DisplayName = "Min Frequency Modulation Amount", EditConditionHides))
	float MinFrequencyModulation = -12.0f;

	// The frequency modulation value (in semitones from the filter frequency) to use when the envelope is loudest
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = ModulationParameters, meta = (EditCondition = "FilterParam == ESourceEffectFilterParam::FilterFrequency", DisplayName = "Max Frequency Modulation Amount", EditConditionHides))
	float MaxFrequencyModulation = 12.0f;

	// The resonance modulation value to use when the envelope is quietest
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = ModulationParameters, meta = (ClampMin = "0.0", ClampMax = "10.0", UIMin = "0", UIMax = "10.0", EditCondition = "FilterParam == ESourceEffectFilterParam::FilterResonance", DisplayName = "Min Resonance Modulation Amount", EditConditionHides))
	float MinResonanceModulation = 0.2f;

	// The resonance modulation value to use when the envelope is loudest
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = ModulationParameters, meta = (ClampMin = "0.0", ClampMax = "10.0", UIMin = "0", UIMax = "10.0", EditCondition = "FilterParam == ESourceEffectFilterParam::FilterResonance", DisplayName = "Max Resonance Modulation Amount", EditConditionHides))
	float MaxResonanceModulation = 3.0f;
};

USTRUCT(BlueprintType)
struct FSourceEffectFilterSettings
{
	GENERATED_USTRUCT_BODY()

	// The type of filter circuit to use.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Filter, meta = (DisplayName = "Circuit"))
	ESourceEffectFilterCircuit FilterCircuit = ESourceEffectFilterCircuit::StateVariable;

	// The type of filter to use.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Filter, meta = (DisplayName = "Type"))
	ESourceEffectFilterType FilterType = ESourceEffectFilterType::LowPass;

	// The filter cutoff frequency
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Filter, meta = (DisplayName = "Cutoff Frequency (hz)", ClampMin = "20.0", UIMin = "20.0", UIMax = "12000.0"))
	float CutoffFrequency = 800.0f;

	// The filter resonance.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Filter, meta = (DisplayName = "Resonance (Q)", ClampMin = "0.5", ClampMax = "10.0", UIMin = "0.5", UIMax = "10.0"))
	float FilterQ = 2.0f;

	// Audio bus settings to use to modulate the filter frequency cutoff (auto-wah) with arbitrary audio from an audio bus
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Filter, meta = (ShowOnlyInnerProperties))
	TArray<FSourceEffectFilterAudioBusModulationSettings> AudioBusModulation;
};

class FSourceEffectFilter : public FSoundEffectSource
{
public:
	UE_API FSourceEffectFilter();
	UE_API virtual ~FSourceEffectFilter();

	// Called on an audio effect at initialization on main thread before audio processing begins.
	UE_API virtual void Init(const FSoundEffectSourceInitData& InitData) override;
	
	// Called when an audio effect preset is changed
	UE_API virtual void OnPresetChanged() override;

	// Process the input block of audio. Called on audio thread.
	UE_API virtual void ProcessAudio(const FSoundEffectSourceInputData& InData, float* OutAudioBufferData) override;

protected:
	UE_API void UpdateFilter();

	Audio::FStateVariableFilter StateVariableFilter;
	Audio::FLadderFilter LadderFilter;
	Audio::FOnePoleFilter OnePoleFilter;
	Audio::IFilter* CurrentFilter;

	float SampleRate;
	float CutoffFrequency;
	float BaseCutoffFrequency;
	float FilterQ;
	float BaseFilterQ;
	ESourceEffectFilterCircuit CircuitType;
	ESourceEffectFilterType FilterType;

	Audio::FAlignedFloatBuffer ScratchModBuffer;
	Audio::FAlignedFloatBuffer ScratchEnvFollowerBuffer;

	struct FAudioBusModulationData
	{
		Audio::FPatchOutputStrongPtr AudioBusPatch;
		Audio::FEnvelopeFollower AudioBusEnvelopeFollower;

		ESourceEffectFilterParam FilterParam;
		float MinFreqModValue = 0.0f;
		float MaxFreqModValue = 0.0f;
		float MinResModValue = 0.0f;
		float MaxResModValue = 0.0f;
		float EnvelopeGain = 0.0f;
	};
	TArray<FAudioBusModulationData> ModData;

	float AudioInput[2];
	float AudioOutput[2];
	int32 NumChannels;
	uint32 AudioDeviceId;
};

UCLASS(MinimalAPI, ClassGroup = AudioSourceEffect, AutoExpandCategories = (SourceEffect), meta = (BlueprintSpawnableComponent))
class USourceEffectFilterPreset : public USoundEffectSourcePreset
{
	GENERATED_BODY()

public:
	EFFECT_PRESET_METHODS(SourceEffectFilter)

	virtual FColor GetPresetColor() const override { return FColor(139.0f, 152.0f, 98.0f); }

	UFUNCTION(BlueprintCallable, Category = "Audio|Effects")
	UE_API void SetSettings(const FSourceEffectFilterSettings& InSettings);

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = SourceEffectPreset, meta = (ShowOnlyInnerProperties))
	FSourceEffectFilterSettings Settings;
};

#undef UE_API
