// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Sound/SoundEffectSource.h"
#include "DSP/FoldbackDistortion.h"
#include "SourceEffectFoldbackDistortion.generated.h"

#define UE_API SYNTHESIS_API

USTRUCT(BlueprintType)
struct FSourceEffectFoldbackDistortionSettings
{
	GENERATED_USTRUCT_BODY()

	// The amount of gain to add to input to allow forcing the triggering of the threshold
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Distortion, meta = (DisplayName = "Input Gain (dB)", ClampMin = "0.0", ClampMax = "60.0", UIMin = "0.0", UIMax = "20.0"))
	float InputGainDb = 0.0f;

	// If the audio amplitude is higher than this, it will fold back
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Distortion, meta = (DisplayName = "Threshold (dB)", ClampMin = "-90.0", ClampMax = "0.0", UIMin = "-60.0", UIMax = "0.0"))
	float ThresholdDb = -6.0f;

	// The amount of gain to apply to the output
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Distortion, meta = (DisplayName = "Output Gain (dB)", ClampMin = "-90.0", ClampMax = "20.0", UIMin = "-60.0", UIMax = "20.0"))
	float OutputGainDb = -3.0f;
};

class FSourceEffectFoldbackDistortion : public FSoundEffectSource
{
public:
	// Called on an audio effect at initialization on main thread before audio processing begins.
	UE_API virtual void Init(const FSoundEffectSourceInitData& InitData) override;
	
	// Called when an audio effect preset is changed
	UE_API virtual void OnPresetChanged() override;

	// Process the input block of audio. Called on audio thread.
	UE_API virtual void ProcessAudio(const FSoundEffectSourceInputData& InData, float* OutAudioBufferData);

protected:
	Audio::FFoldbackDistortion FoldbackDistortion;
};

UCLASS(MinimalAPI, ClassGroup = AudioSourceEffect, meta = (BlueprintSpawnableComponent))
class USourceEffectFoldbackDistortionPreset : public USoundEffectSourcePreset
{
	GENERATED_BODY()

public:
	EFFECT_PRESET_METHODS(SourceEffectFoldbackDistortion)

	virtual FColor GetPresetColor() const override { return FColor(56.0f, 225.0f, 156.0f); }

	UFUNCTION(BlueprintCallable, Category = "Audio|Effects")
	UE_API void SetSettings(const FSourceEffectFoldbackDistortionSettings& InSettings);

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = SourceEffectPreset, meta = (ShowOnlyInnerProperties))
	FSourceEffectFoldbackDistortionSettings Settings;
};

#undef UE_API
