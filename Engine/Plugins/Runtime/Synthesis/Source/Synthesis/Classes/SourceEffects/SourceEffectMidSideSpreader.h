// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Sound/SoundEffectSource.h"
#include "SourceEffectMidSideSpreader.generated.h"

#define UE_API SYNTHESIS_API

// Stereo channel mode
UENUM(BlueprintType)
enum class EStereoChannelMode : uint8
{
		MidSide,
		LeftRight,
		count UMETA(Hidden)
};

// ========================================================================
// FSourceEffectMidSideSpreaderSettings
// This is the source effect's setting struct. 
// ========================================================================

USTRUCT(BlueprintType)
struct FSourceEffectMidSideSpreaderSettings
{
	GENERATED_USTRUCT_BODY()

	// Amount of Mid/Side Spread. 0.0 is no spread, 1.0 is full wide. 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MidSideSpreader, meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float SpreadAmount = 0.5f;

	// Indicate the channel mode of the input signal
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MidSideSpreader)
	EStereoChannelMode InputMode = EStereoChannelMode::LeftRight;

	// Indicate the channel mode of the output signal
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MidSideSpreader)
	EStereoChannelMode OutputMode = EStereoChannelMode::LeftRight;

	// Indicate whether an equal power relationship between the mid and side channels should be maintained
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MidSideSpreader)
	bool bEqualPower = false;
};

// ========================================================================
// FSourceEffectMidSideSpreader
// This is the instance of the source effect. Performs DSP calculations.
// ========================================================================

class FSourceEffectMidSideSpreader : public FSoundEffectSource
{
public:
	UE_API FSourceEffectMidSideSpreader();

	// Called on an audio effect at initialization on main thread before audio processing begins.
	UE_API virtual void Init(const FSoundEffectSourceInitData& InInitData) override;
	
	// Called when an audio effect preset is changed
	UE_API virtual void OnPresetChanged() override;

	// Process the input block of audio. Called on audio thread.
	UE_API virtual void ProcessAudio(const FSoundEffectSourceInputData& InData, float* OutAudioBufferData) override;

protected:

	float MidScale;
	float SideScale;

	int32 NumChannels;

	FSourceEffectMidSideSpreaderSettings SpreaderSettings;

};

// ========================================================================
// USourceEffectMidSideSpreaderPreset
// This code exposes your preset settings and effect class to the editor.
// And allows for a handle to setting/updating effect settings dynamically.
// ========================================================================

UCLASS(MinimalAPI, ClassGroup = AudioSourceEffect, meta = (BlueprintSpawnableComponent))
class USourceEffectMidSideSpreaderPreset : public USoundEffectSourcePreset
{
	GENERATED_BODY()

public:
	// Macro which declares and implements useful functions.
	EFFECT_PRESET_METHODS(SourceEffectMidSideSpreader)

	// Allows you to customize the color of the preset in the editor.
	virtual FColor GetPresetColor() const override { return FColor(126.0f, 180.0f, 255.0f); }

	// Change settings of your effect from blueprints. Will broadcast changes to active instances.
	UFUNCTION(BlueprintCallable, Category = "Audio|Effects")
	UE_API void SetSettings(const FSourceEffectMidSideSpreaderSettings& InSettings);
	
	// The copy of the settings struct. Can't be written to in BP, but can be read.
	// Note that the value read in BP is the serialized settings, will not reflect dynamic changes made in BP.
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = SourceEffectPreset, meta = (ShowOnlyInnerProperties))
	FSourceEffectMidSideSpreaderSettings Settings;
};

#undef UE_API
