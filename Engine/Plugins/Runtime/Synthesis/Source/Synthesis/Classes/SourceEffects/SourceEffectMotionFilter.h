// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Sound/SoundEffectSource.h"
#include "DSP/Filter.h"
#include "SourceEffectMotionFilter.generated.h"

#define UE_API SYNTHESIS_API

namespace Audio { class FLinearEase; }

UENUM(BlueprintType)
enum class ESourceEffectMotionFilterModSource : uint8
{
	// Uunits between Listener and Sound Source.
	DistanceFromListener = 0,

	// Uunits per second change in distance between Listener and Sound Source.
	SpeedRelativeToListener,

	// Uunits per second change in world location of Sound Source.
	SpeedOfSourceEmitter,

	// Uunits per second change in world location of Listener.
	SpeedOfListener,

	// Degrees per second change in Angle of Source from Listener.
	SpeedOfAngleDelta,
	Count UMETA(Hidden)
};

UENUM(BlueprintType)
enum class ESourceEffectMotionFilterModDestination : uint8
{
	// Filter input frequencies range between 20.0f and 15000.0f.
	FilterACutoffFrequency = 0 UMETA(DisplayName = "Filter A Cutoff Frequency (hz)"),

	// Filter input resonances range between 0.5f and 10.0f.
	FilterAResonance UMETA(DisplayName = "Filter A Resonance (Q)"),

	// Filter output dB range between 10.0f and -96.0f. Final Filter output is clamped to +6 dB, use positive values with caution.
	FilterAOutputVolumeDB UMETA(DisplayName = "Filter A Output Volume (dB)"),

	// Filter input frequencies range between 20.0f and 15000.0f.
	FilterBCutoffFrequency UMETA(DisplayName = "Filter B Cutoff Frequency (hz)"),

	// Filter input resonances range between 0.5f and 10.0f.
	FilterBResonance UMETA(DisplayName = "Filter B Resonance (Q)"),

	// Filter output dB range between 10.0f and -96.0f. Final Filter output is clamped to +6 dB, use positive values with caution.
	FilterBOutputVolumeDB UMETA(DisplayName = "Filter B Output Volume (dB)"),

	// Filter Mix values range from -1.0f (Filter A) and 1.0f (Filter B).
	FilterMix,
	Count UMETA(Hidden)
};

UENUM(BlueprintType)
enum class ESourceEffectMotionFilterTopology : uint8
{
	SerialMode = 0,
	ParallelMode,
	Count UMETA(Hidden)
};

UENUM(BlueprintType)
enum class ESourceEffectMotionFilterCircuit : uint8
{
	OnePole = 0,
	StateVariable,
	Ladder,
	Count UMETA(Hidden)
};

UENUM(BlueprintType)
enum class ESourceEffectMotionFilterType : uint8
{
	LowPass = 0,
	HighPass,
	BandPass,
	BandStop,
	Count UMETA(Hidden)
};

USTRUCT(BlueprintType)
struct FSourceEffectIndividualFilterSettings
{
	GENERATED_USTRUCT_BODY()

	// The type of filter circuit to use.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Filter, meta = (DisplayName = "Circuit"))
	ESourceEffectMotionFilterCircuit FilterCircuit = ESourceEffectMotionFilterCircuit::Ladder;

	// The type of filter to use.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Filter, meta = (DisplayName = "Type"))
	ESourceEffectMotionFilterType FilterType = ESourceEffectMotionFilterType::LowPass;

	// The filter cutoff frequency
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Filter, meta = (DisplayName = "Cutoff Frequency (hz)", ClampMin = "20.0", UIMin = "20.0", UIMax = "12000.0"))
	float CutoffFrequency = 800.0f;

	// The filter resonance.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Filter, meta = (DisplayName = "Resonance (Q)", ClampMin = "0.5", ClampMax = "10.0", UIMin = "0.5", UIMax = "10.0"))
	float FilterQ = 2.0f;
};


USTRUCT(BlueprintType)
struct FSourceEffectMotionFilterModulationSettings
{
	GENERATED_USTRUCT_BODY()

	// The Modulation Source
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MotionFilterModulation)
	ESourceEffectMotionFilterModSource ModulationSource = ESourceEffectMotionFilterModSource::DistanceFromListener;

	// The Modulation Clamped Input Range
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MotionFilterModulation)
	FVector2D ModulationInputRange = { 0.0f, 1.0f };

	// The Modulation Random Minimum Output Range
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MotionFilterModulation)
	FVector2D ModulationOutputMinimumRange = { 0.0f, 0.0f };

	// The Modulation Random Maximum Output Range
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MotionFilterModulation)
	FVector2D ModulationOutputMaximumRange  = { 1.0f, 1.0f };

	// Update Ease Speed in milliseconds
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MotionFilterModulation, meta = (DisplayName = "Update Ease (ms)", ClampMin = "0.0", UIMin = "0.0"))
	float UpdateEaseMS = 50.0f;
};

// ========================================================================
// FSourceEffectMotionFilterSettings
// This is the source effect's setting struct. 
// ========================================================================

USTRUCT(BlueprintType)
struct FSourceEffectMotionFilterSettings
{
	GENERATED_USTRUCT_BODY()

	// In Serial Mode, Filter A will process then Filter B will process; in Parallel mode, Filter A and Filter B will process the dry input seprately, then be mixed together afterward.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MotionFilter)
	ESourceEffectMotionFilterTopology MotionFilterTopology = ESourceEffectMotionFilterTopology::ParallelMode;

	// Filter Mix controls the amount of each filter in the signal where -1.0f outputs Only Filter A, 0.0f is an equal balance between Filter A and B, and 1.0f outputs only Filter B. How this blend works depends on the Topology.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MotionFilter, meta = (ClampMin = "-1.0", ClampMax = "1.0", UIMin = "-1.0", UIMax = "1.0"))
	float MotionFilterMix = 0.0f;

	// Initial settings for Filter A
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MotionFilter, meta = (DisplayName = "Filter A"))
	FSourceEffectIndividualFilterSettings FilterASettings;

	// Initial settings for Filter B
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MotionFilter, meta = (DisplayName = "Filter B"))
	FSourceEffectIndividualFilterSettings FilterBSettings;

	// Modulation Mappings
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MotionFilter)
	TMap<ESourceEffectMotionFilterModDestination,FSourceEffectMotionFilterModulationSettings> ModulationMappings;

	// Dry volume pass-through in dB. 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MotionFilter, meta = (DisplayName = "Dry Volume (dB)", ClampMin = "-96.0", UIMin = "-96.0", UIMax = "10.0"))
	float DryVolumeDb = -96.0f;

	FSourceEffectMotionFilterSettings()
	{
		ModulationMappings.Empty((uint8)ESourceEffectMotionFilterModDestination::Count);
	}
};

// ========================================================================
// FMotionFilter
// This is the struct of an individual Motion Filter.
// It contains all the information needed to track the state 
// of a single Motion Filter
// ========================================================================


struct FMotionFilter
{
	// Filter Settings
	Audio::FOnePoleFilter OnePoleFilter;
	Audio::FStateVariableFilter StateVarFilter;
	Audio::FLadderFilter LadderFilter;
	// Which filter we're currently using
	Audio::IFilter* CurrentFilter;

	ESourceEffectMotionFilterCircuit CurrentFilterCircuit;

	// Filter Type
	Audio::EFilter::Type FilterType;

	float FilterFrequency;
	float FilterQ;

	FMotionFilter()
		: CurrentFilter(nullptr)
		, CurrentFilterCircuit(ESourceEffectMotionFilterCircuit::OnePole)
		, FilterType(Audio::EFilter::LowPass)
		, FilterFrequency(800.0f)
		, FilterQ(2.0f)
	{}
};

// ========================================================================
// FSourceEffectMotionFilter
// This is the instance of the source effect. Performs DSP calculations.
// ========================================================================


class FSourceEffectMotionFilter : public FSoundEffectSource
{
public:
	UE_API FSourceEffectMotionFilter();

	// Called on an audio effect at initialization on main thread before audio processing begins.
	UE_API virtual void Init(const FSoundEffectSourceInitData& InInitData) override;
	
	// Called when an audio effect preset is changed
	UE_API virtual void OnPresetChanged() override;

	// Process the input block of audio. Called on audio thread.
	UE_API virtual void ProcessAudio(const FSoundEffectSourceInputData& InData, float* OutAudioBufferData) override;

protected:
	// Motion filter topology
	ESourceEffectMotionFilterTopology Topology;

	FMotionFilter MotionFilterA;
	FMotionFilter MotionFilterB;

	// Filter Mix
	float FilterMixAmount;

	// Mod Map
	TMap<ESourceEffectMotionFilterModDestination, FSourceEffectMotionFilterModulationSettings> ModMap;

	// Mod Map Random Output Range
	TMap<ESourceEffectMotionFilterModDestination, FVector2D> ModMapOutputRange;

	// Current Mod Matrix comprised of [Source] x [Destination] coordinates
	TArray<TArray<float>> ModMatrix;

	// Target values for the Mod Matrix
	TArray<TArray<float>> TargetMatrix;

	// Last Target values for the Mod Matrix
	TArray<TArray<float>> LastTargetMatrix;

	// Linear Ease Matrix
	TArray<TArray<Audio::FLinearEase>> LinearEaseMatrix;

	// Linear Ease Matrix is Initialized
	TArray<TArray<bool>> LinearEaseMatrixInit;

	// Attenuation of sound in linear units
	float DryVolumeScalar;

	// Modulation Sources
	TArray<float> ModSources;

	// This is the last time Mod Source data has been updated
	double ModSourceTimeStamp;

	float LastDistance;
	FVector LastEmitterWorldPosition;
	FVector LastListenerWorldPosition;
	FVector LastEmitterNormalizedPosition;

	// Base Destination Values
	TArray<float> BaseDestinationValues;

	// Modulation Destination Values
	TArray<float> ModDestinationValues;

	// Modulation Destination Values
	TArray<float> ModDestinationUpdateTimeMS;

	// Intermediary Scratch Buffers
	Audio::FAlignedFloatBuffer ScratchBufferA;
	Audio::FAlignedFloatBuffer ScratchBufferB;

	// Filter Output Scalars
	float FilterAMixScale;
	float FilterBMixScale;
	float FilterAOutputScale;
	float FilterBOutputScale;

	// Update Filter Parameters
	UE_API void UpdateFilter(FMotionFilter* MotionFilter, ESourceEffectMotionFilterCircuit FilterCircuitType, ESourceEffectMotionFilterType MotionFilterType, float FilterFrequency, float FilterQ);

	// Applies modulation changes to Filter based on Destination Input Values
	UE_API void ApplyFilterModulation(const TArray<float>& DestinationSettings);

	// Update Modulation Source Parameters
	UE_API void UpdateModulationSources(const FSoundEffectSourceInputData& InData);

	// Updates Modulated Parameters, returns true if parameters were updated
	UE_API bool UpdateModulationMatrix(const float UpdateTime);

	// Updates Modulation Destinations based on updated Matrix Values
	UE_API void UpdateModulationDestinations();

	// Sample Rate cached
	float SampleRate;

	// Number of channels in source
	int32 NumChannels;

	// SampleRate * NumChannels
	float ChannelRate;
};

// ========================================================================
// USourceEffectMotionFilterPreset
// This code exposes your preset settings and effect class to the editor.
// And allows for a handle to setting/updating effect settings dynamically.
// ========================================================================

UCLASS(MinimalAPI, ClassGroup = AudioSourceEffect, meta = (BlueprintSpawnableComponent))
class USourceEffectMotionFilterPreset : public USoundEffectSourcePreset
{
	GENERATED_BODY()

public:
	// Macro which declares and implements useful functions.
	EFFECT_PRESET_METHODS(SourceEffectMotionFilter)

	// Allows you to customize the color of the preset in the editor.
	virtual FColor GetPresetColor() const override { return FColor(0.0f, 185.0f, 211.0f); }

	// Change settings of your effect from blueprints. Will broadcast changes to active instances.
	UFUNCTION(BlueprintCallable, Category = "Audio|Effects")
	UE_API void SetSettings(const FSourceEffectMotionFilterSettings& InSettings);
	
	// The copy of the settings struct. Can't be written to in BP, but can be read.
	// Note that the value read in BP is the serialized settings, will not reflect dynamic changes made in BP.
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = SourceEffectPreset, meta = (ShowOnlyInnerProperties))
	FSourceEffectMotionFilterSettings Settings;
};

#undef UE_API
