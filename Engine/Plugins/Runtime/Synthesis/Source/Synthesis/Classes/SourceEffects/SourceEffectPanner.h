// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Sound/SoundEffectSource.h"
#include "SourceEffectPanner.generated.h"

#define UE_API SYNTHESIS_API

USTRUCT(BlueprintType)
struct FSourceEffectPannerSettings
{
	GENERATED_USTRUCT_BODY()

	// The spread of the source. 1.0 means left only in left channel, right only in right; 0.0 means both mixed, -1.0 means right and left channels are inverted.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Panner, meta = (ClampMin = "-1.0", ClampMax = "1.0", UIMin = "-1.0", UIMax = "1.0"))
	float Spread = 1.0f;

	// The pan of the source. -1.0 means left, 0.0 means center, 1.0 means right.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Panner, meta = (ClampMin = "-1.0", ClampMax = "1.0", UIMin = "-1.0", UIMax = "1.0"))
	float Pan = 0.0f;
};

class FSourceEffectPanner : public FSoundEffectSource
{
public:
	// Called on an audio effect at initialization on main thread before audio processing begins.
	UE_API virtual void Init(const FSoundEffectSourceInitData& InitData) override;

	// Called when an audio effect preset is changed
	UE_API virtual void OnPresetChanged() override;

	// Process the input block of audio. Called on audio render thread.
	UE_API virtual void ProcessAudio(const FSoundEffectSourceInputData& InData, float* OutAudioBufferData) override;

	FSourceEffectPanner()
	{
		for (int32 i = 0; i < 2; ++i)
		{
			SpreadGains[i] = 0.0f;
			PanGains[i] = 0.0f;
		}
		NumChannels = 0;
	}

protected:

	// The pan value of the source effect
	float SpreadGains[2];
	float PanGains[2];
	int32 NumChannels;
};

UCLASS(MinimalAPI, ClassGroup = AudioSourceEffect, meta = (BlueprintSpawnableComponent))
class USourceEffectPannerPreset : public USoundEffectSourcePreset
{
	GENERATED_BODY()

public:
	EFFECT_PRESET_METHODS(SourceEffectPanner)

	virtual FColor GetPresetColor() const override { return FColor(127.0f, 155.0f, 101.0f); }

	UFUNCTION(BlueprintCallable, Category = "Audio|Effects")
	UE_API void SetSettings(const FSourceEffectPannerSettings& InSettings);
	
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = SourceEffectPreset, meta = (ShowOnlyInnerProperties))
	FSourceEffectPannerSettings Settings;
};

#undef UE_API
