// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Sound/SoundEffectSource.h"
#include "DSP/Phaser.h"
#include "SourceEffectPhaser.generated.h"

#define UE_API SYNTHESIS_API

UENUM(BlueprintType)
enum class EPhaserLFOType : uint8
{
	Sine = 0,
	UpSaw,
	DownSaw,
	Square,
	Triangle,
	Exponential,
	RandomSampleHold,
	Count UMETA(Hidden)
};

USTRUCT(BlueprintType)
struct FSourceEffectPhaserSettings
{
	GENERATED_USTRUCT_BODY()

	// The wet level of the phaser effect
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Phaser, meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float WetLevel = 0.2f;

	// The LFO frequency of the phaser effect
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Phaser, meta = (ClampMin = "0.0", UIMin = "0.0", UIMax = "5.0"))
	float Frequency = 2.0f;

	// The feedback of the phaser effect
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Phaser, meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float Feedback = 0.3f;

	// The phaser LFO type
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Phaser, meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	EPhaserLFOType LFOType = EPhaserLFOType::Sine;

	// Whether or not to use quadtrature phase for the LFO modulation
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Phaser, meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	bool UseQuadraturePhase = false;
};

class FSourceEffectPhaser : public FSoundEffectSource
{
public:
	// Called on an audio effect at initialization on main thread before audio processing begins.
	UE_API virtual void Init(const FSoundEffectSourceInitData& InitData) override;
	
	// Called when an audio effect preset is changed
	UE_API virtual void OnPresetChanged() override;

	// Process the input block of audio. Called on audio thread.
	UE_API virtual void ProcessAudio(const FSoundEffectSourceInputData& InData, float* OutAudioBufferData) override;

protected:
	Audio::FPhaser Phaser;
};

UCLASS(MinimalAPI, ClassGroup = AudioSourceEffect, meta = (BlueprintSpawnableComponent))
class USourceEffectPhaserPreset : public USoundEffectSourcePreset
{
	GENERATED_BODY()

public:
	EFFECT_PRESET_METHODS(SourceEffectPhaser)

	virtual FColor GetPresetColor() const override { return FColor(140.0f, 4.0f, 4.0f); }

	UFUNCTION(BlueprintCallable, Category = "Audio|Effects")
	UE_API void SetSettings(const FSourceEffectPhaserSettings& InSettings);

	// The depth of the chorus effect
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SourceEffectPreset, meta = (ShowOnlyInnerProperties))
	FSourceEffectPhaserSettings Settings;
};

#undef UE_API
