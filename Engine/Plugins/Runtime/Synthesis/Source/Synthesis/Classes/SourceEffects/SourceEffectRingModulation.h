// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Sound/SoundEffectSource.h"
#include "DSP/RingModulation.h"
#include "SourceEffectRingModulation.generated.h"

#define UE_API SYNTHESIS_API

class UAudioBus;

UENUM(BlueprintType)
enum class ERingModulatorTypeSourceEffect : uint8
{
	Sine,
	Saw,
	Triangle,
	Square,
	Count UMETA(Hidden)
};

USTRUCT(BlueprintType)
struct FSourceEffectRingModulationSettings
{
	GENERATED_USTRUCT_BODY()

	// Ring modulation modulator oscillator type
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = RingModulation, meta = (ClampMin = "5.0", UIMin = "10.0", UIMax = "5000.0"))
	ERingModulatorTypeSourceEffect ModulatorType = ERingModulatorTypeSourceEffect::Sine;

	// Ring modulation frequency
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = RingModulation, meta = (DisplayName = "Frequency (hz)", ClampMin = "5.0", UIMin = "10.0", UIMax = "5000.0"))
	float Frequency = 100.0f;

	// Ring modulation depth
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = RingModulation, meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float Depth = 0.5f;

	// Gain for the dry signal (no ring mod)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = RingModulation, meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float DryLevel = 0.0f;

	// Gain for the wet signal (with ring mod)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = RingModulation, meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float WetLevel = 1.0f;

	// Audio bus to use to modulate the effect
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = RingModulation)
	TObjectPtr<UAudioBus> AudioBusModulator;
};

class FSourceEffectRingModulation : public FSoundEffectSource
{
public:
	// Called on an audio effect at initialization on main thread before audio processing begins.
	UE_API virtual void Init(const FSoundEffectSourceInitData& InitData) override;
	
	// Called when an audio effect preset is changed
	UE_API virtual void OnPresetChanged() override;

	// Process the input block of audio. Called on audio thread.
	UE_API virtual void ProcessAudio(const FSoundEffectSourceInputData& InData, float* OutAudioBufferData) override;

protected:
	Audio::FRingModulation RingModulation;
	uint32 AudioDeviceId;
	int32 NumChannels = 0;
};

UCLASS(MinimalAPI, ClassGroup = AudioSourceEffect, meta = (BlueprintSpawnableComponent))
class USourceEffectRingModulationPreset : public USoundEffectSourcePreset
{
	GENERATED_BODY()

public:
	EFFECT_PRESET_METHODS(SourceEffectRingModulation)

	virtual FColor GetPresetColor() const override { return FColor(122.0f, 125.0f, 195.0f); }

	UFUNCTION(BlueprintCallable, Category = "Audio|Effects")
	UE_API void SetSettings(const FSourceEffectRingModulationSettings& InSettings);

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = SourceEffectPreset, meta = (ShowOnlyInnerProperties))
	FSourceEffectRingModulationSettings Settings;
};

#undef UE_API
