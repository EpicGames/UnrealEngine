// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Sound/SoundEffectSource.h"
#include "DSP/Delay.h"
#include "SourceEffectSimpleDelay.generated.h"

#define UE_API SYNTHESIS_API

USTRUCT(BlueprintType)
struct FSourceEffectSimpleDelaySettings
{
	GENERATED_USTRUCT_BODY()

	// Speed of sound in meters per second when using distance-based delay
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Delay, meta = (DisplayAfter = "bDelayBasedOnDistance", ClampMin = "0.01", UIMin = "0.01", UIMax = "10000.0", EditCondition = "bDelayBasedOnDistance"))
	float SpeedOfSound = 343.0f;

	// Delay amount in seconds
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Delay, meta = (DisplayAfter = "SpeedOfSound", ClampMin = "0.0", ClampMax = "2.0", UIMin = "0.0", UIMax = "2.0", EditCondition = "!bDelayBasedOnDistance"))
	float DelayAmount = 0.0f;

	// Gain stage on dry (non-delayed signal)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Delay, meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float DryAmount = 0.0f;

	// Gain stage on wet (delayed) signal
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Delay, meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float WetAmount = 1.0f;

	// Amount to feedback into the delay line (because why not)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Delay, meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float Feedback = 0.0f;

	// Whether or not to delay the audio based on the distance to the listener or use manual delay
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Delay)
	uint32 bDelayBasedOnDistance : 1;

	// Whether or not to allow the attenuation distance override value vs the distance to listener to be used for distance-based delay.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Delay)
	uint32 bUseDistanceOverride : 1;
	
	FSourceEffectSimpleDelaySettings()
		: bDelayBasedOnDistance(true)
		, bUseDistanceOverride(true)
	{}
};

class FSourceEffectSimpleDelay : public FSoundEffectSource
{
public:
	// Called on an audio effect at initialization on main thread before audio processing begins.
	UE_API virtual void Init(const FSoundEffectSourceInitData& InitData) override;
	
	// Called when an audio effect preset is changed
	UE_API virtual void OnPresetChanged() override;

	// Process the input block of audio. Called on audio thread.
	UE_API virtual void ProcessAudio(const FSoundEffectSourceInputData& InData, float* OutAudioBufferData) override;

protected:
	// 2 Delay lines, one for each channel
	TArray<Audio::FDelay> Delays;
	TArray<float> FeedbackSamples;
	FSourceEffectSimpleDelaySettings SettingsCopy;
	bool bIsInit = true;
};

UCLASS(MinimalAPI, ClassGroup = AudioSourceEffect, meta = (BlueprintSpawnableComponent))
class USourceEffectSimpleDelayPreset : public USoundEffectSourcePreset
{
	GENERATED_BODY()

public:
	EFFECT_PRESET_METHODS(SourceEffectSimpleDelay)

	virtual FColor GetPresetColor() const override { return FColor(100.0f, 165.0f, 85.0f); }

	UFUNCTION(BlueprintCallable, Category = "Audio|Effects")
	UE_API void SetSettings(const FSourceEffectSimpleDelaySettings& InSettings);

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = SourceEffectPreset, meta = (ShowOnlyInnerProperties))
	FSourceEffectSimpleDelaySettings Settings;
};

#undef UE_API
