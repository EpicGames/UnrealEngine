// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DSP/DelayStereo.h"
#include "Sound/SoundEffectSource.h"
#include "SourceEffectStereoDelay.generated.h"

#define UE_API SYNTHESIS_API

UENUM(BlueprintType)
enum class EStereoDelaySourceEffect : uint8
{
	// Left input mixes with left delay line output and feeds to left output. 
	// Right input mixes with right delay line output and feeds to right output.
	Normal = 0,

	// Left input mixes right delay line output and feeds to right output.
	// Right input mixes with left delay line output and feeds to left output.
	Cross,

	// Left input mixes with left delay line output and feeds to right output.
	// Right input mixes with right delay line output and feeds to left output.
	PingPong,

	Count UMETA(Hidden)
};

// Stereo delay filter types
UENUM(BlueprintType)
enum class EStereoDelayFiltertype  : uint8
{
	Lowpass = 0,
	Highpass,
	Bandpass,
	Notch,

	Count UMETA(Hidden)
};

USTRUCT(BlueprintType)
struct FSourceEffectStereoDelaySettings
{
	GENERATED_USTRUCT_BODY()

	// What mode to set the stereo delay effect.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = StereoDelay, meta = (DisplayName = "Mode"))
	EStereoDelaySourceEffect DelayMode = EStereoDelaySourceEffect::PingPong;

	// The base amount of delay in the left and right channels of the delay line.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = StereoDelay, meta = (DisplayName = "Delay Time (ms)", ClampMin = "0.0", ClampMax = "2000.0", UIMin = "0.0", UIMax = "2000.0"))
	float DelayTimeMsec = 500.0f;

	// The amount of audio to feedback into the delay line once the delay has been tapped.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = StereoDelay, meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float Feedback = 0.1f;

	// Delay spread for left and right channels. Allows left and right channels to have differential delay amounts. Useful for stereo channel decorrelation.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = StereoDelay, meta = (ClampMin = "-1.0", ClampMax = "1.0", UIMin = "-1.0", UIMax = "1.0"))
	float DelayRatio = 0.2f;

	// The amount of delay effect to mix with the dry input signal into the effect.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = StereoDelay, meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float WetLevel = 0.4f;

	// The dry level of the effect
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = StereoDelay, meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float DryLevel = 1.0f;

	// Whether or not to enable filtering
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Filter, meta = (DisplayName = "Enabled"))
	bool bFilterEnabled = false;

	// Filter type to feed feedback audio to
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Filter, meta = (DisplayName = "Type", EditCondition = "bFilterEnabled"))
	EStereoDelayFiltertype FilterType = EStereoDelayFiltertype::Lowpass;

	// Cutoff frequency of the filter
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Filter, meta = (DisplayName = "Cutoff Frequency (hz)", ClampMin = "20.0", ClampMax = "20000", UIMin = "20.0", UIMax = "20000", EditCondition = "bFilterEnabled"))
	float FilterFrequency = 20000.0f;

	// Q of the filter
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Filter, meta = (DisplayName = "Resonance (Q)", ClampMin = "0.5", ClampMax = "10.0", UIMin = "0.5", UIMax = "10.0", EditCondition = "bFilterEnabled"))
	float FilterQ = 2.0f;
};

class FSourceEffectStereoDelay : public FSoundEffectSource
{
public:
	// Called on an audio effect at initialization on main thread before audio processing begins.
	UE_API virtual void Init(const FSoundEffectSourceInitData& InitData) override;
	
	// Called when an audio effect preset is changed
	UE_API virtual void OnPresetChanged() override;

	// Process the input block of audio. Called on audio thread.
	UE_API virtual void ProcessAudio(const FSoundEffectSourceInputData& InData, float* OutAudioBufferData) override;

protected:
	Audio::FDelayStereo DelayStereo;
};

UCLASS(MinimalAPI, ClassGroup = AudioSourceEffect, meta = (BlueprintSpawnableComponent))
class USourceEffectStereoDelayPreset : public USoundEffectSourcePreset
{
	GENERATED_BODY()

public:

	EFFECT_PRESET_METHODS(SourceEffectStereoDelay)

	virtual FColor GetPresetColor() const override { return FColor(23.0f, 121.0f, 225.0f); }

	UFUNCTION(BlueprintCallable, Category = "Audio|Effects")
	UE_API void SetSettings(const FSourceEffectStereoDelaySettings& InSettings);

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = SourceEffectPreset, meta = (ShowOnlyInnerProperties))
	FSourceEffectStereoDelaySettings Settings;
};

#undef UE_API
