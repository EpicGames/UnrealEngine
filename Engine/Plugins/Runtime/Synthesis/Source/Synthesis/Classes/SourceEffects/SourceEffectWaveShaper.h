// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Sound/SoundEffectSource.h"
#include "DSP/WaveShaper.h"
#include "SourceEffectWaveShaper.generated.h"

#define UE_API SYNTHESIS_API

USTRUCT(BlueprintType)
struct FSourceEffectWaveShaperSettings
{
	GENERATED_USTRUCT_BODY()

	// The amount of wave shaping. 0.0 = no wave shaping.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = WaveShaper, meta = (ClampMin = "0.0", UIMin = "0.0", UIMax = "500.0"))
	float Amount = 1.0f;

	// The amount of wave shaping. 0.0 = no wave shaping.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = WaveShaper, meta = (DisplayName = "Output Gain (dB)", ClampMin = "-60.0", ClampMax = "20.0", UIMin = "-60.0", UIMax = "20.0"))
	float OutputGainDb = 0.0f;
};

class FSourceEffectWaveShaper : public FSoundEffectSource
{
public:
	// Called on an audio effect at initialization on main thread before audio processing begins.
	UE_API virtual void Init(const FSoundEffectSourceInitData& InitData) override;
	
	// Called when an audio effect preset is changed
	UE_API virtual void OnPresetChanged() override;

	// Process the input block of audio. Called on audio thread.
	UE_API virtual void ProcessAudio(const FSoundEffectSourceInputData& InData, float* OutAudioBufferData) override;

protected:
	Audio::FWaveShaper WaveShaper;
	int32 NumChannels;
};

UCLASS(MinimalAPI, ClassGroup = AudioSourceEffect, meta = (BlueprintSpawnableComponent))
class USourceEffectWaveShaperPreset : public USoundEffectSourcePreset
{
	GENERATED_BODY()

public:
	EFFECT_PRESET_METHODS(SourceEffectWaveShaper)

	virtual FColor GetPresetColor() const override { return FColor(218.0f, 248.0f, 78.0f); }

	UFUNCTION(BlueprintCallable, Category = "Audio|Effects")
	UE_API void SetSettings(const FSourceEffectWaveShaperSettings& InSettings);

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = SourceEffectPreset, meta = (ShowOnlyInnerProperties))
	FSourceEffectWaveShaperSettings Settings;
};

#undef UE_API
