// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Sound/SoundEffectSubmix.h"
#include "SubmixEffectConvolutionReverb.generated.h"

#define UE_API SYNTHESIS_API

class UAudioImpulseResponse;
class USubmixEffectConvolutionReverbPreset;
enum class ESubmixEffectConvolutionReverbBlockSize : uint8;
namespace Audio { class FEffectConvolutionReverb; }
namespace AudioConvReverbIntrinsics { struct FVersionData; }

USTRUCT(BlueprintType)
struct FSubmixEffectConvolutionReverbSettings
{
	GENERATED_USTRUCT_BODY()

	UE_API FSubmixEffectConvolutionReverbSettings();

	/* Used to account for energy added by convolution with "loud" Impulse Responses. 
	 * This value is not directly editable in the editor because it is copied from the 
	 * associated UAudioImpulseResponse. */
	UPROPERTY();
	float NormalizationVolumeDb;

	// Controls how much of the wet signal is mixed into the output, in Decibels
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SubmixEffectPreset, meta = (DisplayName = "Wet Volume (dB)", ClampMin = "-96.0", ClampMax = "0.0"));
	float WetVolumeDb;

	// Controls how much of the dry signal is mixed into the output, in Decibels
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SubmixEffectPreset, meta = (DisplayName = "Dry Volume (dB)", ClampMin = "-96.0", ClampMax = "0.0"));
	float DryVolumeDb;

	/* If true, input audio is directly routed to output audio with applying any effect. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SubmixEffectPreset)
	bool bBypass;

	/* If true, the submix input audio is downmixed to match the IR asset audio channel
	 * format. If false, the input audio's channels are matched to the IR assets
	 * audio channels.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SubmixEffectPreset, meta = (EditCondition = "!bBypass"))
	bool bMixInputChannelFormatToImpulseResponseFormat;

	/* If true, the reverberated audio is upmixed or downmixed to match the submix 
	 * output audio format. If false, the reverberated audio's channels are matched
	 * to the submixs output audio channels. 
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SubmixEffectPreset, meta = (EditCondition = "!bBypass"))
	bool bMixReverbOutputToOutputChannelFormat;

	/* Amount of audio to be sent to rear channels in quad/surround configurations */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SubmixEffectPreset|Surround", meta = (DisplayName = "Surround Rear Channel Bleed (dB)", EditCondition = "bMixReverbOutputToOutputChannelFormat && !bBypass", ClampMin = "-60.0", UIMin = "-60.0", ClampMax = "15.0", UIMax = "15.0"))
	float SurroundRearChannelBleedDb;

	/* If true, rear channel bleed sends will have their phase inverted. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SumixEffectPreset|Surround", meta = (EditCondition = "bMixReverbOutputToOutputChannelFormat && !bBypass"))
	bool bInvertRearChannelBleedPhase;

	/* If true, send Surround Rear Channel Bleed Amount sends front left to back right and vice versa */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SubmixEffectPreset|Surround", meta = (EditCondition = "bMixReverbOutputToOutputChannelFormat && !bBypass"))
	bool bSurroundRearChannelFlip;

	UPROPERTY(meta = ( DeprecatedProperty ) )
	float SurroundRearChannelBleedAmount_DEPRECATED;

	UPROPERTY(meta = ( DeprecatedProperty ) )
	TObjectPtr<UAudioImpulseResponse> ImpulseResponse_DEPRECATED;

	UPROPERTY(meta = ( DeprecatedProperty ) )
	bool AllowHardwareAcceleration_DEPRECATED;
};


/** Audio render thread effect object. */
class FSubmixEffectConvolutionReverb : public FSoundEffectSubmix
{
public:
	
	FSubmixEffectConvolutionReverb() = delete;
	// Construct a convolution object with an existing preset. 
	UE_API FSubmixEffectConvolutionReverb(const USubmixEffectConvolutionReverbPreset* InPreset);

	UE_API ~FSubmixEffectConvolutionReverb();

	// Called on an audio effect at initialization on main thread before audio processing begins.
	UE_API virtual void Init(const FSoundEffectSubmixInitData& InitData) override;

	// Called when an audio effect preset settings is changed.
	UE_API virtual void OnPresetChanged() override;

	// Process the input block of audio. Called on audio thread.
	UE_API virtual void OnProcessAudio(const FSoundEffectSubmixInputData& InData, FSoundEffectSubmixOutputData& OutData) override;

	// Call on the game thread in order to update the impulse response and hardware acceleration
	// used in this submix effect.
	UE_API AudioConvReverbIntrinsics::FVersionData UpdateConvolutionReverb(const USubmixEffectConvolutionReverbPreset* InPreset);

	UE_API void RebuildConvolutionReverb();

	float WetVolume = 1.f;
	float DryVolume = 1.f;

private:
	// Sets current runtime settings for convolution reverb which do *not* trigger
	// a FConvolutionReverb rebuild.  These settings will be applied to FConvolutionReverb 
	// at the next call to UpdateParameters()
	UE_API void SetConvolutionReverbParameters(const FSubmixEffectConvolutionReverbSettings& InSettings);

	// Reverb performs majority of DSP operations
	TSharedRef<Audio::FEffectConvolutionReverb> Reverb;
};

UCLASS(MinimalAPI)
class USubmixEffectConvolutionReverbPreset : public USoundEffectSubmixPreset
{
	GENERATED_BODY()

public:
	
	UE_API USubmixEffectConvolutionReverbPreset(const FObjectInitializer& ObjectInitializer);

	/* Note: The SubmixEffect boilerplate macros could not be utilized here because
	 * the "CreateNewEffect" implementation differed from those available in the
	 * boilerplate macro.
	 */

	UE_API virtual bool CanFilter() const override;

	UE_API virtual bool HasAssetActions() const;

	UE_API virtual FText GetAssetActionName() const override;

	UE_API virtual UClass* GetSupportedClass() const override;

	UE_API virtual FSoundEffectBase* CreateNewEffect() const override;

	UE_API virtual USoundEffectPreset* CreateNewPreset(UObject* InParent, FName Name, EObjectFlags Flags) const override;

	UE_API virtual void Init() override;


	UE_API FSubmixEffectConvolutionReverbSettings GetSettings() const;

	/** Set the convolution reverb settings */
	UFUNCTION(BlueprintCallable, Category = "Audio|Effects")
	UE_API void SetSettings(const FSubmixEffectConvolutionReverbSettings& InSettings);

	/** Set the convolution reverb impulse response */
	UFUNCTION(BlueprintCallable, BlueprintSetter, Category = "Audio|Effects")
	UE_API void SetImpulseResponse(UAudioImpulseResponse* InImpulseResponse);

	/** The impulse response used for convolution. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter = SetImpulseResponse, Category = SubmixEffectPreset)
	TObjectPtr<UAudioImpulseResponse> ImpulseResponse;

	/** ConvolutionReverbPreset Preset Settings. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter = SetSettings, Category = SubmixEffectPreset, meta = (ShowOnlyInnerProperties))
	FSubmixEffectConvolutionReverbSettings Settings;


	/** Set the internal block size. This can effect latency and performance. Higher values will result in
	 * lower CPU costs while lower values will result higher CPU costs. Latency may be affected depending
	 * on the interplay between audio engines buffer sizes and this effects block size. Generally, higher
	 * values result in higher latency, and lower values result in lower latency. 
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, AdvancedDisplay, Category = SubmixEffectPreset)
	ESubmixEffectConvolutionReverbBlockSize BlockSize;

	/** Opt into hardware acceleration of the convolution reverb (if available) */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, AdvancedDisplay, Category = SubmixEffectPreset)
	bool bEnableHardwareAcceleration;
	

#if WITH_EDITORONLY_DATA
	// Binds to the UAudioImpulseRespont::OnObjectPropertyChanged delegate of the current ImpulseResponse
	UE_API void BindToImpulseResponseObjectChange();

	UE_API virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	// Called when a property changes on the ImpulseResponse object
	UE_API void PostEditChangeImpulseProperty(FPropertyChangedEvent& PropertyChangedEvent);
#endif

	UE_API virtual void PostLoad() override;

private:
	void SetImpulseResponseSettings(UAudioImpulseResponse* InImpulseResponse);

	void UpdateSettings();

	void UpdateDeprecatedProperties();

	// This method requires that the submix effect is registered with a preset.  If this 
	// submix effect is not registered with a preset, then this will not update the convolution
	// algorithm.
	void RebuildConvolutionReverb();

	mutable FCriticalSection SettingsCritSect; 
	FSubmixEffectConvolutionReverbSettings SettingsCopy; 

#if WITH_EDITORONLY_DATA

	TMap<UObject*, FDelegateHandle> DelegateHandles;
#endif
};

#undef UE_API
