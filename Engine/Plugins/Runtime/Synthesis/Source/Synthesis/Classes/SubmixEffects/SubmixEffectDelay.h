// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DSP/Dsp.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Sound/SoundEffectSubmix.h"
#include "SubmixEffectDelay.generated.h"

#define UE_API SYNTHESIS_API

namespace Audio { class FDelay; }

// ========================================================================
// FSubmixEffectDelaySettings
// UStruct used to define user-exposed params for use with your effect.
// ========================================================================

USTRUCT(BlueprintType)
struct FSubmixEffectDelaySettings
{
	GENERATED_USTRUCT_BODY()

	// Maximum possible length for a delay, in milliseconds. Changing this at runtime will reset the effect.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Initialization, meta = (DisplayName = "Maximum Delay Length (ms)", ClampMin = "10.0", UIMin = "10.0", UIMax = "20000.0"))
	float MaximumDelayLength = 2000.0f;

	// Number of milliseconds over which a tap will reach it's set length and gain. Smaller values are more responsive, while larger values will make pitching less dramatic.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Realtime, meta = (DisplayName = "Interpolation Time (ms)", ClampMin = "0.0", UIMin = "0.0", UIMax = "20000.0"))
	float InterpolationTime = 400.0f;

	// Number of milliseconds of delay.  Caps at max delay at runtime.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Realtime, meta = (DisplayName = "Delay Length (ms)", ClampMin = "0.0", UIMin = "0.0", UIMax = "20000.0"))
	float DelayLength = 1000.0f;
};

UCLASS(MinimalAPI)
class USubmixEffectDelayStatics : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category = "Audio|Effects|Delay")
	static FSubmixEffectDelaySettings& SetMaximumDelayLength(UPARAM(ref) FSubmixEffectDelaySettings& DelaySettings, float MaximumDelayLength)
	{
		DelaySettings.MaximumDelayLength = FMath::Max(0.0f, MaximumDelayLength);
		DelaySettings.DelayLength = FMath::Min(DelaySettings.MaximumDelayLength, DelaySettings.DelayLength);
		return DelaySettings;
	}

	UFUNCTION(BlueprintCallable, Category = "Audio|Effects|Delay")
	static FSubmixEffectDelaySettings& SetInterpolationTime(UPARAM(ref) FSubmixEffectDelaySettings& DelaySettings, float InterpolationTime)
	{
		DelaySettings.InterpolationTime = FMath::Max(0.0f, InterpolationTime);
		return DelaySettings;
	}

	UFUNCTION(BlueprintCallable, Category = "Audio|Effects|Delay")
	static FSubmixEffectDelaySettings& SetDelayLength(UPARAM(ref) FSubmixEffectDelaySettings& DelaySettings, float DelayLength)
	{
		DelaySettings.DelayLength = FMath::Max(0.0f, DelayLength);
		DelaySettings.MaximumDelayLength = FMath::Max(DelaySettings.MaximumDelayLength, DelaySettings.DelayLength);

		return DelaySettings;
	}
};

class FSubmixEffectDelay : public FSoundEffectSubmix
{
public:
	UE_API FSubmixEffectDelay();
	UE_API ~FSubmixEffectDelay();

	//~ Begin FSoundEffectSubmix
	UE_API virtual void Init(const FSoundEffectSubmixInitData& InData) override;
	UE_API virtual void OnProcessAudio(const FSoundEffectSubmixInputData& InData, FSoundEffectSubmixOutputData& OutData) override;
	//~ End FSoundEffectSubmix

	//~ Begin FSoundEffectBase
	UE_API virtual void OnPresetChanged() override;
	//~End FSoundEffectBase

	// Sets the reverb effect parameters based from audio thread code
	UE_API void SetEffectParameters(const FSubmixEffectDelaySettings& InTapEffectParameters);

	// Set the time it takes, in milliseconds, to arrive at a new parameter.
	UE_API void SetInterpolationTime(float Time);

	// Set how long the actually delay is, in milliseconds.
	UE_API void SetDelayLineLength(float Length);

private:
	static const float MinLengthDelaySec;

	// This is called on the audio render thread to query the parameters.
	void UpdateParameters();

	// Called on the audio render thread when the number of channels is changed
	void OnNumChannelsChanged(int32 NumChannels);

	// Params struct used to pass parameters safely to the audio render thread.
	Audio::TParams<FSubmixEffectDelaySettings> Params;

	// Sample rate cached at initialization. Used to gage interpolation times.
	float SampleRate;

	// Current maximum delay line length, in milliseconds.
	float MaxDelayLineLength;

	// Current interpolation time, in seconds.
	float InterpolationTime;

	// Most recently set delay line length.
	float TargetDelayLineLength;

	Audio::FLinearEase InterpolationInfo;

	// Delay lines for each channel.
	TArray<Audio::FDelay> DelayLines;
};

// ========================================================================
// USubmixEffectDelayPreset
// Class which processes audio streams and uses parameters defined in the preset class.
// ========================================================================

UCLASS(MinimalAPI)
class USubmixEffectDelayPreset : public USoundEffectSubmixPreset
{
	GENERATED_BODY()

public:
	EFFECT_PRESET_METHODS(SubmixEffectDelay)

	// Sets runtime delay settings. This will replace any dynamically added or modified settings without modifying
	// the original UObject.
	UFUNCTION(BlueprintCallable, Category = "Audio|Effects|Delay", meta = (DisplayName = "Set Dynamic Settings"))
	UE_API void SetSettings(const FSubmixEffectDelaySettings& InSettings);

	// Sets object's default settings. This will update both the default UObject settings (and mark it as dirty),
	// as well as any dynamically set settings.
	UFUNCTION(BlueprintCallable, Category = "Audio|Effects|Delay")
	UE_API void SetDefaultSettings(const FSubmixEffectDelaySettings& InSettings);

	// Get the maximum delay possible.
	UFUNCTION(BlueprintCallable, Category = "Audio|Effects|Delay")
	float GetMaxDelayInMilliseconds() const
	{
		return DynamicSettings.MaximumDelayLength;
	};

	// Set the time it takes to interpolate between parameters, in milliseconds.
	UFUNCTION(BlueprintCallable, Category = "Audio|Effects|Delay")
	UE_API void SetInterpolationTime(float Time);

	// Set how long the delay actually is, in milliseconds.
	UFUNCTION(BlueprintCallable, Category = "Audio|Effects|Delay")
	UE_API void SetDelay(float Length);

public:
	UE_API virtual void OnInit() override;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter = SetDefaultSettings, Category = SubmixEffectPreset, Meta = (ShowOnlyInnerProperties))
	FSubmixEffectDelaySettings Settings;

	UPROPERTY(transient)
	FSubmixEffectDelaySettings DynamicSettings;
};

#undef UE_API
