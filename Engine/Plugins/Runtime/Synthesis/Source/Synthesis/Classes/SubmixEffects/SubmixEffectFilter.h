// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DSP/Filter.h"
#include "Sound/SoundEffectSubmix.h"
#include "SubmixEffectFilter.generated.h"

#define UE_API SYNTHESIS_API

UENUM(BlueprintType)
enum class ESubmixFilterType : uint8
{
	LowPass = 0,
	HighPass,
	BandPass,
	BandStop,
	Count UMETA(Hidden)
};

UENUM(BlueprintType)
enum class ESubmixFilterAlgorithm : uint8
{
	OnePole = 0,
	StateVariable,
	Ladder,
	Count UMETA(Hidden)
};

// ========================================================================
// FSubmixEffectFilterSettings
// UStruct used to define user-exposed params for use with your effect.
// ========================================================================

USTRUCT(BlueprintType)
struct FSubmixEffectFilterSettings
{
	GENERATED_USTRUCT_BODY()

	// What type of filter to use for the submix filter effect
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Filter, meta = (DisplayName = "Type"))
	ESubmixFilterType FilterType = ESubmixFilterType::LowPass;

	// What type of filter algorithm to use for the submix filter effect
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Filter, meta = (DisplayName = "Circuit"))
	ESubmixFilterAlgorithm FilterAlgorithm = ESubmixFilterAlgorithm::OnePole;

	// The output filter cutoff frequency (hz) [0.0, 20000.0]
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Filter, meta = (DisplayName = "Cutoff Frequency (hz)", ClampMin = "0.0", ClampMax = "20000.0", UIMin = "0.0", UIMax = "20000.0"))
	float FilterFrequency = 20000.0f;

	// The output filter resonance (Q) [0.5, 10]
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Filter, meta = (DisplayName = "Resonance (Q)", ClampMin = "0.5", ClampMax = "10.0", UIMin = "0.5", UIMax = "10.0"))
	float FilterQ = 2.0f;
};

class FSubmixEffectFilter : public FSoundEffectSubmix
{
public:
	UE_API FSubmixEffectFilter();
	virtual ~FSubmixEffectFilter() = default;

	//~ Begin FSoundEffectSubmix
	UE_API virtual void Init(const FSoundEffectSubmixInitData& InData) override;
	UE_API virtual void OnProcessAudio(const FSoundEffectSubmixInputData& InData, FSoundEffectSubmixOutputData& OutData) override;
	//~ End FSoundEffectSubmix

	//~ Begin FSoundEffectBase
	UE_API virtual void OnPresetChanged() override;
	//~End FSoundEffectBase

	// Sets the filter type
	UE_API void SetFilterType(ESubmixFilterType InType);

	// Sets the filter algorithm
	UE_API void SetFilterAlgorithm(ESubmixFilterAlgorithm InAlgorithm);

	// Sets the base filter cutoff frequency
	UE_API void SetFilterCutoffFrequency(float InFrequency);

	// Sets the mod filter cutoff frequency
	UE_API void SetFilterCutoffFrequencyMod(float InFrequency);

	// Sets the filter Q
	UE_API void SetFilterQ(float InQ);

	// Sets the filter Q
	UE_API void SetFilterQMod(float InQ);

private:

	UE_API void InitFilter();

	// Sample rate of the submix effect
	float SampleRate;

	// Filters for each channel and type
	Audio::FOnePoleFilter OnePoleFilter;
	Audio::FStateVariableFilter StateVariableFilter;
	Audio::FLadderFilter LadderFilter;

	// The current filter selected
	Audio::IFilter* CurrentFilter;

	// Filter control data
	ESubmixFilterAlgorithm FilterAlgorithm;
	ESubmixFilterType FilterType;

	float FilterFrequency;
	float FilterFrequencyMod;

	float FilterQ;
	float FilterQMod;

	int32 NumChannels;
};

// ========================================================================
// USubmixEffectFilterPreset
// Class which processes audio streams and uses parameters defined in the preset class.
// ========================================================================

UCLASS(MinimalAPI)
class USubmixEffectFilterPreset : public USoundEffectSubmixPreset
{
	GENERATED_BODY()

public:
	EFFECT_PRESET_METHODS(SubmixEffectFilter)

	// Set all filter effect settings
	UFUNCTION(BlueprintCallable, Category = "Audio|Effects|Filter")
	UE_API void SetSettings(const FSubmixEffectFilterSettings& InSettings);

	// Sets the filter type
	UFUNCTION(BlueprintCallable, Category = "Audio|Effects|Filter")
	UE_API void SetFilterType(ESubmixFilterType InType);

	// Sets the filter algorithm
	UFUNCTION(BlueprintCallable, Category = "Audio|Effects|Filter")
	UE_API void SetFilterAlgorithm(ESubmixFilterAlgorithm InAlgorithm);

	// Sets the base filter cutoff frequency
	UFUNCTION(BlueprintCallable, Category = "Audio|Effects|Filter")
	UE_API void SetFilterCutoffFrequency(float InFrequency);

	// Sets the mod filter cutoff frequency
	UFUNCTION(BlueprintCallable, Category = "Audio|Effects|Filter")
	UE_API void SetFilterCutoffFrequencyMod(float InFrequency);

	// Sets the filter Q
	UFUNCTION(BlueprintCallable, Category = "Audio|Effects|Filter")
	UE_API void SetFilterQ(float InQ);

	// Sets the filter Q
	UFUNCTION(BlueprintCallable, Category = "Audio|Effects|Filter")
	UE_API void SetFilterQMod(float InQ);

public:

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SubmixEffectPreset, meta = (ShowOnlyInnerProperties))
	FSubmixEffectFilterSettings Settings;
};

#undef UE_API
