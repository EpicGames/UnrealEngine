// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Flexiverb.h"
#include "Sound/SoundEffectSubmix.h"
#include "SubmixEffectFlexiverb.generated.h"

#define UE_API SYNTHESIS_API

USTRUCT(BlueprintType)
struct FSubmixEffectFlexiverbSettings
{
	GENERATED_USTRUCT_BODY()
	
	/** PreDelay - 0.01 < 10.0 < 40.0 - Amount of delay to the first echo in milliseconds. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = ReverbParameters, meta = (DisplayName = "Pre Delay (ms)", ClampMin = "0.01", ClampMax = "30"))
	float PreDelay;

	/** Time in seconds it will take for the impulse response to decay to -60 dB. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = ReverbParameters, meta = (DisplayName = "Decay Time (seconds)", ClampMin = "0.4", ClampMax = "5.0"))
	float DecayTime;

	/** Room Dampening - 0.0 < 0.85 < 1.0 - Frequency at which the room dampens.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = ReverbParameters, meta = (ClampMin = "60.0", ClampMax = "12000.0"))
	float RoomDampening;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = ReverbParameters, meta = (ClampMin = "2", ClampMax = "6"))
	int32 Complexity;

	FSubmixEffectFlexiverbSettings()
	: PreDelay(10.0f)
	, DecayTime(7.0f)
	, RoomDampening(220.0f)
	, Complexity(2)
	{
	}
};

class FSubmixEffectFlexiverb : public FSoundEffectSubmix
{
public:
	UE_API FSubmixEffectFlexiverb();

	// Called on an audio effect at initialization on main thread before audio processing begins.
	UE_API virtual void Init(const FSoundEffectSubmixInitData& InSampleRate) override;
	
	// Called when an audio effect preset is changed
	UE_API virtual void OnPresetChanged() override;

	// We want to receive downmixed submix audio to stereo input for the reverb effect
	virtual uint32 GetDesiredInputChannelCountOverride() const override { return 2; }

	// Process the input block of audio. Called on audio thread.
	UE_API virtual void OnProcessAudio(const FSoundEffectSubmixInputData& InData, FSoundEffectSubmixOutputData& OutData) override;

	// Sets the reverb effect parameters based from audio thread code
	UE_API void SetEffectParameters(const Audio::FFlexiverbSettings& InReverbEffectParameters);

private:
	UE_API void UpdateParameters();

	// The reverb effect
	Audio::FFlexiverb Flexiverb;

	// The reverb effect params
	Audio::TParams<Audio::FFlexiverbSettings> Params;

	bool bIsEnabled;
};

UCLASS(MinimalAPI)
class USubmixEffectFlexiverbPreset : public USoundEffectSubmixPreset
{
	GENERATED_BODY()

public:
	EFFECT_PRESET_METHODS(SubmixEffectFlexiverb)

	UFUNCTION(BlueprintCallable, Category = "Audio|Effects")
	UE_API void SetSettings(const FSubmixEffectFlexiverbSettings& InSettings);

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SubmixEffectPreset, meta =(ShowOnlyInnerProperties))
	FSubmixEffectFlexiverbSettings Settings;
};

#undef UE_API
