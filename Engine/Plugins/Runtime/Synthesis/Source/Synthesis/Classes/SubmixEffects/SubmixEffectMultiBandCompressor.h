// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DSP/LinkwitzRileyBandSplitter.h"
#include "SubmixEffects/AudioMixerSubmixEffectDynamicsProcessor.h"
#include "SubmixEffectMultiBandCompressor.generated.h"

#define UE_API SYNTHESIS_API

namespace Audio { class FDynamicsProcessor; }

USTRUCT(BlueprintType)
struct FDynamicsBandSettings
{
	GENERATED_BODY()

	// Frequency of the crossover between this band and the next. The last band will have this property ignored
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Response, meta = (ClampMin = "20.0", ClampMax = "20000.0", UIMin = "20.0", UIMax = "20000"))
	float CrossoverTopFrequency = 20000.f;

	// The amount of time to ramp into any dynamics processing effect in milliseconds.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Response, meta = (ClampMin = "1.0", ClampMax = "300.0", UIMin = "1.0", UIMax = "200.0"))
	float AttackTimeMsec = 10.f;

	// The amount of time to release the dynamics processing effect in milliseconds
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Response, meta = (ClampMin = "20.0", ClampMax = "5000.0", UIMin = "20.0", UIMax = "5000.0"))
	float ReleaseTimeMsec = 100.f;

	// The threshold at which to perform a dynamics processing operation
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Dynamics, meta = (ClampMin = "-72.0", ClampMax = "0.0", UIMin = "-72.0", UIMax = "0.0"))
	float ThresholdDb = -6.f;

	// The dynamics processor ratio -- has different meaning depending on the processor type.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Dynamics, meta = (ClampMin = "1.0", ClampMax = "20.0", UIMin = "1.0", UIMax = "20.0"))
	float Ratio = 1.5f;

	// The knee bandwidth of the compressor to use in dB
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Dynamics, meta = (DisplayName = "Knee (dB)", ClampMin = "0.0", ClampMax = "20.0", UIMin = "0.0", UIMax = "20.0"))
	float KneeBandwidthDb = 10.f;

	// The input gain of the dynamics processor in dB
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Gain, meta = (DisplayName = "Input Gain (dB)", ClampMin = "-12.0", ClampMax = "20.0", UIMin = "-12.0", UIMax = "20.0"))
	float InputGainDb = 0.f;

	// The output gain of the dynamics processor in dB
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Gain, meta = (DisplayName = "Output Gain (dB)", ClampMin = "-12.0", ClampMax = "20.0", UIMin = "-12.0", UIMax = "20.0"))
	float OutputGainDb = 0.f;
};

// A submix dynamics processor
USTRUCT(BlueprintType)
struct FSubmixEffectMultibandCompressorSettings
{
	GENERATED_BODY()

	// Controls how each band will react to audio above its threshold
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = General, meta = (DisplayName = "Type"))
	ESubmixEffectDynamicsProcessorType DynamicsProcessorType = ESubmixEffectDynamicsProcessorType::Compressor;

	// Controls how quickly the bands will react to a signal above its threshold
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Dynamics)
	ESubmixEffectDynamicsPeakMode PeakMode = ESubmixEffectDynamicsPeakMode::RootMeanSquared;

	// Whether to compress all channels equally, and how to evaluate the overall level
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Dynamics)
	ESubmixEffectDynamicsChannelLinkMode LinkMode = ESubmixEffectDynamicsChannelLinkMode::Average;

	// The amount of time to look ahead of the current audio. Allows for transients to be included in dynamics processing.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Response, meta = (DisplayName = "Look Ahead (ms)", ClampMin = "0.0", ClampMax = "50.0", UIMin = "0.0", UIMax = "50.0"))
	float LookAheadMsec = 3.f;

	// Toggles treating the attack and release envelopes as analog-style vs digital-style. Analog will respond a bit more naturally/slower.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Response)
	bool bAnalogMode = true;

	// Turning off FourPole mode will use cheaper, shallower 2-pole crossovers
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Crossover)
	bool bFourPole = true;

	// Whether or not to bypass effect
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = General, meta = (DisplayName = "Bypass", DisplayAfter = "DynamicsProcessorType"))
	bool bBypass = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Sidechain, meta = (EditCondition = "!bBypass"))
	ESubmixEffectDynamicsKeySource KeySource = ESubmixEffectDynamicsKeySource::Default;

	// If set, uses output of provided submix as modulator of input signal for dynamics processor (Uses input signal as default modulator)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Sidechain, meta = (EditCondition = "!bBypass && KeySource == ESubmixEffectDynamicsKeySource::AudioBus", EditConditionHides))
	TObjectPtr<UAudioBus> ExternalAudioBus = nullptr;

	// If set, uses output of provided submix as modulator of input signal for dynamics processor (Uses input signal as default modulator)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Sidechain, meta = (EditCondition = "!bBypass && KeySource == ESubmixEffectDynamicsKeySource::Submix", EditConditionHides))
	TObjectPtr<USoundSubmix> ExternalSubmix = nullptr;

	// Gain to apply to key signal if external input is supplied
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Sidechain, meta = (DisplayName = "External Input Gain (dB)", EditCondition = "ExternalSubmix != nullptr || ExternalAudioBus != nullptr", UIMin = "-60.0", UIMax = "30.0"))
	float KeyGainDb = 0.0f;

	// Audition the key modulation signal, bypassing enveloping and processing the input signal.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Sidechain, meta = (DisplayName = "Key Audition", EditCondition = "!bBypass"))
	bool bKeyAudition = false;

	// Each band is a full dynamics processor, affecting at a unique frequency range
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Bands)
	TArray<FDynamicsBandSettings> Bands;
};

class FSubmixEffectMultibandCompressor : public FSoundEffectSubmix
{
public:
	UE_API FSubmixEffectMultibandCompressor();

	UE_API virtual ~FSubmixEffectMultibandCompressor();

	// Gets the effect's deviceId that owns it
	UE_API Audio::FDeviceId GetDeviceId() const;

	// Called on an audio effect at initialization on main thread before audio processing begins.
	UE_API virtual void Init(const FSoundEffectSubmixInitData& InSampleRate) override;

	// Process the input block of audio. Called on audio thread.
	UE_API virtual void OnProcessAudio(const FSoundEffectSubmixInputData& InData, FSoundEffectSubmixOutputData& OutData) override;

	// Called when an audio effect preset is changed
	UE_API virtual void OnPresetChanged() override;

	// called from OnPresetChanged when something is changed that needs extra attention
	UE_API void Initialize(FSubmixEffectMultibandCompressorSettings& Settings);

protected:
	// ISubmixBufferListener interface
	UE_API void ResetKey();
	UE_API void UpdateKeyFromSettings(const FSubmixEffectMultibandCompressorSettings& InSettings);
	UE_API bool UpdateKeySourcePatch();

	UE_API void OnDeviceCreated(Audio::FDeviceId InDeviceId);
	UE_API void OnDeviceDestroyed(Audio::FDeviceId InDeviceId);

	UE_API Audio::FMixerDevice* GetMixerDevice();

	Audio::FAlignedFloatBuffer AudioExternal;

private:
	static constexpr int32 MaxBlockNumSamples = 8192; // 8 channels * 1024 samples
	static_assert((MaxBlockNumSamples % 4) == 0, "MaxBlockNumSamples must be evenly divisible by 4 to allow for SIMD optimization" );
	FKeySource KeySource;

	Audio::FDeviceId DeviceId;

	FDelegateHandle DeviceCreatedHandle;
	FDelegateHandle DeviceDestroyedHandle;

	Audio::FAlignedFloatBuffer ScratchBuffer;

	TArray<Audio::FDynamicsProcessor> DynamicsProcessors;

	Audio::FLinkwitzRileyBandSplitter KeyBandSplitter;
	Audio::FMultibandBuffer KeyMultiBandBuffer;

	Audio::FLinkwitzRileyBandSplitter BandSplitter;
	Audio::FMultibandBuffer MultiBandBuffer;

	// cached crossover + band info, so we can check if they need a re-build when editing
	int32 PrevNumBands = 0;
	TArray<float> PrevCrossovers;
	bool bPrevFourPole = true;

	int32 NumChannels = 0;
	int32 FrameSize = 0;
	float SampleRate = 48000.f;

	bool bInitialized = false;
	bool bBypass = false;

	friend class USubmixEffectMultibandCompressorPreset;
};

UCLASS(MinimalAPI, ClassGroup = AudioSourceEffect, meta = (BlueprintSpawnableComponent))
class USubmixEffectMultibandCompressorPreset : public USoundEffectSubmixPreset
{
	GENERATED_BODY()

public:
	EFFECT_PRESET_METHODS(SubmixEffectMultibandCompressor)

	UE_API virtual void OnInit() override;

#if WITH_EDITOR
	UE_API virtual void PostEditChangeChainProperty(struct FPropertyChangedChainEvent& InChainEvent) override;
#endif // WITH_EDITOR

	UFUNCTION(BlueprintCallable, Category = "Audio|Effects")
	UE_API void ResetKey();

	// Sets the source key input as the provided AudioBus' output.  If no object is provided, key is set
	// to effect's input.
	UFUNCTION(BlueprintCallable, Category = "Audio|Effects")
	UE_API void SetAudioBus(UAudioBus* AudioBus);

	// Sets the source key input as the provided Submix's output.  If no object is provided, key is set
	// to effect's input.
	UFUNCTION(BlueprintCallable, Category = "Audio|Effects")
	UE_API void SetExternalSubmix(USoundSubmix* Submix);

	UFUNCTION(BlueprintCallable, Category = "Audio|Effects")
	UE_API void SetSettings(const FSubmixEffectMultibandCompressorSettings& InSettings);

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SubmixEffectPreset, meta = (ShowOnlyInnerProperties))
	FSubmixEffectMultibandCompressorSettings Settings;

private:
	void SetKey(ESubmixEffectDynamicsKeySource InKeySource, UObject* InObject, int32 InNumChannels = 0);
};

#undef UE_API
