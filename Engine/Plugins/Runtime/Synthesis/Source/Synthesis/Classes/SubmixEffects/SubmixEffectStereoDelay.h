// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Sound/SoundEffectSubmix.h"
#include "SourceEffects/SourceEffectStereoDelay.h"
#include "SubmixEffectStereoDelay.generated.h"

#define UE_API SYNTHESIS_API


USTRUCT(BlueprintType)
struct FSubmixEffectStereoDelaySettings
{
	GENERATED_USTRUCT_BODY()

	// What mode to set the stereo delay effect.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Delay")
	EStereoDelaySourceEffect DelayMode = EStereoDelaySourceEffect::PingPong;

	// The base amount of delay in the left and right channels of the delay line.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Delay", meta = (ClampMin = "0.0", ClampMax = "2000.0", UIMin = "0.0", UIMax = "2000.0"))
	float DelayTimeMsec = 500.0f;

	// The amount of audio to feedback into the delay line once the delay has been tapped.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Delay", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float Feedback = 0.1f;

	// Delay spread for left and right channels. Allows left and right channels to have differential delay amounts. Useful for stereo channel decorrelation.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Delay", meta = (ClampMin = "-1.0", ClampMax = "1.0", UIMin = "-1.0", UIMax = "1.0"))
	float DelayRatio = 0.2f;

	// The amount of delay effect to mix with the dry input signal into the effect.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Delay", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float WetLevel = 0.4f;

	// The dry level of the effect
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Delay", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float DryLevel = 1.0f;

	// Whether or not to enable filtering
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Delay")
	bool bFilterEnabled = false;

	// Filter type to feed feedback audio to
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Delay", meta = (EditCondition = "bFilterEnabled"))
	EStereoDelayFiltertype FilterType = EStereoDelayFiltertype::Lowpass;

	// Cutoff frequency of the filter
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Delay", meta = (ClampMin = "20.0", ClampMax = "20000", UIMin = "20.0", UIMax = "20000", EditCondition = "bFilterEnabled"))
	float FilterFrequency = 20000.0f;

	// Q of the filter
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Delay", meta = (ClampMin = "0.5", ClampMax = "10.0", UIMin = "0.5", UIMax = "10.0", EditCondition = "bFilterEnabled"))
	float FilterQ = 2.0f;
};

class FSubmixEffectStereoDelay : public FSoundEffectSubmix
{
public:
	UE_API FSubmixEffectStereoDelay();
	UE_API ~FSubmixEffectStereoDelay();

	//~ Begin FSoundEffectSubmix
	UE_API virtual void Init(const FSoundEffectSubmixInitData& InData) override;
	UE_API virtual void OnProcessAudio(const FSoundEffectSubmixInputData& InData, FSoundEffectSubmixOutputData& OutData) override;
	//~ End FSoundEffectSubmix

	//~ Begin FSoundEffectBase
	UE_API virtual void OnPresetChanged() override;
	//~End FSoundEffectBase

private:

	Audio::FAlignedFloatBuffer ScratchStereoBuffer;
	Audio::FDelayStereo DelayStereo;
};

// ========================================================================
// USubmixEffectDelayPreset
// Class which processes audio streams and uses parameters defined in the preset class.
// ========================================================================

UCLASS(MinimalAPI)
class USubmixEffectStereoDelayPreset : public USoundEffectSubmixPreset
{
	GENERATED_BODY()

public:
	EFFECT_PRESET_METHODS(SubmixEffectStereoDelay)

	// Set all tap delay settings. This will replace any dynamically added or modified taps.
	UFUNCTION(BlueprintCallable, Category = "Audio|Effects|Delay")
	UE_API void SetSettings(const FSubmixEffectStereoDelaySettings& InSettings);


public:
	UE_API virtual void OnInit() override;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SubmixEffectPreset, Meta = (ShowOnlyInnerProperties))
	FSubmixEffectStereoDelaySettings Settings;
};

#undef UE_API
