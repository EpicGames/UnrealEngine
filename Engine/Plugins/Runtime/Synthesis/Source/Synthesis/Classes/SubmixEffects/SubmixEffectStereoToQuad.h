// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Sound/SoundEffectSubmix.h"
#include "SubmixEffectStereoToQuad.generated.h"

#define UE_API SYNTHESIS_API


USTRUCT(BlueprintType)
struct FSubmixEffectStereoToQuadSettings
{
	GENERATED_USTRUCT_BODY()

	// Whether or not to flip the left and right input channels when sending to the rear channel.
	// This can be useful to have a stereo field when hearing audio to the left and right in surround output configuration.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "StereoToQuad")
	bool bFlipChannels = false;

	// The gain (in decibels) of the rear channels
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "StereoToQuad")
	float RearChannelGain = 0.0f;
};

class FSubmixEffectStereoToQuad : public FSoundEffectSubmix
{
public:
	UE_API FSubmixEffectStereoToQuad();
	UE_API ~FSubmixEffectStereoToQuad();

	//~ Begin FSoundEffectSubmix
	UE_API virtual void Init(const FSoundEffectSubmixInitData& InData) override;
	UE_API virtual void OnProcessAudio(const FSoundEffectSubmixInputData& InData, FSoundEffectSubmixOutputData& OutData) override;
	//~ End FSoundEffectSubmix

	//~ Begin FSoundEffectBase
	UE_API virtual void OnPresetChanged() override;
	//~End FSoundEffectBase

private:

	FSubmixEffectStereoToQuadSettings CurrentSettings;
	float LinearGain = 1.0f;
	Audio::FAlignedFloatBuffer ScratchStereoBuffer;
};

// Submix effect which sends stereo audio to quad (left surround and right surround) if the channel count is greater than 2.
UCLASS(MinimalAPI)
class USubmixEffectStereoToQuadPreset : public USoundEffectSubmixPreset
{
	GENERATED_BODY()

public:
	EFFECT_PRESET_METHODS(SubmixEffectStereoToQuad)

	// Set all tap delay settings. This will replace any dynamically added or modified taps.
	UFUNCTION(BlueprintCallable, Category = "Audio|Effects|Delay")
	UE_API void SetSettings(const FSubmixEffectStereoToQuadSettings& InSettings);


public:
	UE_API virtual void OnInit() override;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SubmixEffectPreset, Meta = (ShowOnlyInnerProperties))
	FSubmixEffectStereoToQuadSettings Settings;
};

#undef UE_API
