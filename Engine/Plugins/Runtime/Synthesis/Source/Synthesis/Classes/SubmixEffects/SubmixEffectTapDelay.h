// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DSP/Delay.h"
#include "Sound/SoundEffectSubmix.h"
#include "SubmixEffectTapDelay.generated.h"

#define UE_API SYNTHESIS_API

DECLARE_LOG_CATEGORY_EXTERN(LogTapDelay, Log, All);

UENUM()
enum class ETapLineMode : uint8
{
	// Send tap audio output to a channel directly
	SendToChannel,

	// Allow tap to pan between channels based on azimuth angle
	Panning,

	// Disables the tap audio and performs a fadeout
	Disabled
};

struct FTapDelayInterpolationInfo
{
	FTapDelayInterpolationInfo()
	{
	}

	UE_API void Init(float SampleRate);
	
	UE_API void SetGainValue(float Value, float InterpolationTime);
	UE_API float GetGainValue();

	UE_API void SetLengthValue(float Value, float InterpolationTime);
	UE_API float GetLengthValue();

private:
	Audio::FLinearEase LengthParam;
	Audio::FLinearEase GainParam;
};

USTRUCT(BlueprintType)
struct FTapDelayInfo
{
	GENERATED_USTRUCT_BODY()

	// Whether the tap line should send directly to a channel, pan, or not produce sound at all.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Realtime)
	ETapLineMode TapLineMode;

	// Amount of time before this echo is heard in milliseconds.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Realtime, meta = (DisplayName = "Delay Length (ms)", ClampMin = "0.0", ClampMax = "10000"))
	float DelayLength;

	// How loud this echo should be, in decibels.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Realtime, meta = (ClampMin = "-60.0", ClampMax = "6.0"))
	float Gain;

	// When the Tap Line Mode is set to Send To Channel, designates index of channel from which the echo should play.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Realtime, meta = (EditCondition = "TapLineMode == ETapLineMode::SendToChannel", ClampMin = "0", ClampMax = "8"))
	int32 OutputChannel;

	// When the Tap Line Mode is set to Panning, designates the angle at which the echo should be panned.
	// On Surround systems, 0 is directly in front of the listener, -90 is left, 90 is right, and 180/-180 is directly behind the listener.
	// On Stereo systems, <-90 is fully in the left ear, and >90 is fully in the right ear
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Realtime, meta = (EditCondition = "TapLineMode == ETapLineMode::Panning", ClampMin = "-180.0", ClampMax = "180.0"))
	float PanInDegrees;

	UPROPERTY(Transient, meta = (IgnoreForMemberInitializationTest))
	int32 TapId;

public:

	UE_API FTapDelayInfo();
};

// ========================================================================
// FTapDelaySubmixPresetSettings
// UStruct used to define user-exposed params for use with your effect.
// ========================================================================

USTRUCT(BlueprintType)
struct FSubmixEffectTapDelaySettings
{
	GENERATED_USTRUCT_BODY()

	// Maximum possible length for a delay, in milliseconds. Changing this at runtime will reset the effect.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Initialization, meta = (DisplayName = "Maximum Delay Length (ms)", ClampMin = "10.0", UIMin = "10.0", UIMax = "20000.0"))
	float MaximumDelayLength = 10000.0f;

	// Number of milliseconds over which a tap will reach it's set length and gain. Smaller values are more responsive, while larger values will make pitching less dramatic.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Realtime, meta = (DisplayName = "Interpolation Time (ms)", ClampMin = "0.0", UIMin = "0.0", UIMax = "20000.0"))
	float InterpolationTime = 400.0f;

	// Each tap's metadata
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Realtime)
	TArray<FTapDelayInfo> Taps;
};

class FSubmixEffectTapDelay : public FSoundEffectSubmix
{
public:
	UE_API FSubmixEffectTapDelay();

	virtual ~FSubmixEffectTapDelay() = default;

	//~ Begin FSoundEffectSubmix
	UE_API virtual void Init(const FSoundEffectSubmixInitData& InData) override;
	UE_API virtual void OnProcessAudio(const FSoundEffectSubmixInputData& InData, FSoundEffectSubmixOutputData& OutData) override;
	//~ End FSoundEffectSubmix

	//~ Begin FSoundEffectBase
	UE_API virtual void OnPresetChanged() override;
	//~End FSoundEffectBase

	// Sets the reverb effect parameters based from audio thread code
	UE_API void SetEffectParameters(const FSubmixEffectTapDelaySettings& InTapEffectParameters);

	UE_API void AddTap(int32 TapId);
	UE_API void RemoveTap(int32 TapId);
	UE_API void SetTap(int32 TapId, const FTapDelayInfo& DelayInfo);

	// Set the interpolation time
	UE_API void SetInterpolationTime(float Time);

private:

	// This is called on the audio render thread to query the parameters.
	UE_API void UpdateParameters();

	// This is called in UpdateParameters to set up our per-sample parameter interpolation.
	UE_API void UpdateInterpolations();

	// Params struct used to pass parameters safely to the audio render thread.
	Audio::TParams<FSubmixEffectTapDelaySettings> Params;

	// Sample rate cached at initialization. Used to gauge interpolation times.
	float SampleRate;

	// Current maximum delay line length, in milliseconds.
	float MaxDelayLineLength;

	// Current interpolation time, in seconds.
	float InterpolationTime;

	// Target parameters that we interpolate to. Updated directly from params.
	TArray<FTapDelayInfo> TargetTaps;

	// Current state of each tap.
	TArray<FTapDelayInterpolationInfo> CurrentTaps;

	// How many increments we have left before all CurrentTaps reach the TargetTaps
	int32 TapIncrementsRemaining;

	// Whether taps have been modified.
	bool bSettingsModified;

	Audio::FDelay DelayLine;
};

// ========================================================================
// UTapDelaySubmixPreset
// Class which processes audio streams and uses parameters defined in the preset class.
// ========================================================================

UCLASS(MinimalAPI, ClassGroup = AudioSourceEffect, meta = (BlueprintSpawnableComponent), AutoExpandCategories = (Realtime))
class USubmixEffectTapDelayPreset : public USoundEffectSubmixPreset
{
	GENERATED_BODY()

public:
	EFFECT_PRESET_METHODS(SubmixEffectTapDelay)

	// Set all tap delay setting. This will replace any dynamically added or modified taps.
	UFUNCTION(BlueprintCallable, Category = "Audio|Effects|TapDelay")
	UE_API void SetSettings(const FSubmixEffectTapDelaySettings& InSettings);

	// Adds a dynamic tap delay with the given settings. Returns the tap id. 
	UFUNCTION(BlueprintCallable, Category = "Audio|Effects|TapDelay")
	UE_API void AddTap(int32& TapId);

	// Remove the tap from the preset.
	UFUNCTION(BlueprintCallable, Category = "Audio|Effects|TapDelay")
	UE_API void RemoveTap(int32 TapId);

	// Modify a specific tap.
	UFUNCTION(BlueprintCallable, Category = "Audio|Effects|TapDelay")
	UE_API void SetTap(int32 TapId, const FTapDelayInfo& TapInfo);

	// Get the current info about a specific tap.
	UFUNCTION(BlueprintCallable, Category = "Audio|Effects|TapDelay")
	UE_API void GetTap(int32 TapId, FTapDelayInfo& TapInfo);

	// Retrieve an array of all tap ids for the submix effect.
	UFUNCTION(BlueprintCallable, Category = "Audio|Effects|TapDelay")
	UE_API void GetTapIds(TArray<int32>& TapIds);

	// Get the maximum delay possible.
	UFUNCTION(BlueprintCallable, Category = "Audio|Effects|TapDelay")
	float GetMaxDelayInMilliseconds() { return DynamicSettings.MaximumDelayLength;  };

	// Set the time it takes to interpolate between parameters, in milliseconds.
	UFUNCTION(BlueprintCallable, Category = "Audio|Effects|TapDelay")
	UE_API void SetInterpolationTime(float Time);

public:

	UE_API virtual void OnInit() override;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SubmixEffectPreset, Meta = (ShowOnlyInnerProperties))
	FSubmixEffectTapDelaySettings Settings;

	FSubmixEffectTapDelaySettings DynamicSettings;
};

#undef UE_API
