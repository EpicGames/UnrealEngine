// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DataTable.h"
#include "Components/SynthComponent.h"
#include "EpicSynth1.h"
#include "EpicSynth1Component.generated.h"

#define UE_API SYNTHESIS_API

USTRUCT(BlueprintType)
struct FEpicSynth1Patch
{
	GENERATED_USTRUCT_BODY()

	// A modular synth patch source (e.g. LFO1/LFO2/Modulation Envelope)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset")
	ESynth1PatchSource PatchSource = ESynth1PatchSource::LFO1;

	// Patch cables to patch destinations from the patch source.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset")
	TArray<FSynth1PatchCable> PatchCables;
};

USTRUCT(BlueprintType)
struct FModularSynthPreset : public FTableRowBase
{
	GENERATED_USTRUCT_BODY()

	// Whether or not to allow multiple synth voices.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset")
	uint32 bEnablePolyphony : 1;

	// What type of oscillator to use for oscillator 1
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset")
	ESynth1OscType Osc1Type;

	// The linear gain of oscillator 1 [0.0, 1.0]
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float Osc1Gain;

	// The octave of oscillator 1. [-8.0, 8.0]
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset", meta = (ClampMin = "-8.0", ClampMax = "8.0", UIMin = "-8.0", UIMax = "8.0"))
	float Osc1Octave;

	// The semi-tones of oscillator 1. [-12.0, 12.0]
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset", meta = (ClampMin = "-12.0", ClampMax = "12.0", UIMin = "-12.0", UIMax = "12.0"))
	float Osc1Semitones;

	// The cents (hundreds of a semitone) of oscillator 1. [-100.0, 100.0]
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset", meta = (ClampMin = "-100.0", ClampMax = "100.0", UIMin = "-100.0", UIMax = "100.0"))
	float Osc1Cents;

	// The pulsewidth of oscillator 1 (when using a square wave type oscillator). [0.0, 1.0]
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float Osc1PulseWidth;

	// What type of oscillator to use for oscillator 2
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset")
	ESynth1OscType Osc2Type;

	// The linear gain of oscillator 2 [0.0, 1.0]
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float Osc2Gain;

	// The octave of oscillator 2. [-8.0, 8.0]
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset", meta = (ClampMin = "-8.0", ClampMax = "8.0", UIMin = "-8.0", UIMax = "8.0"))
	float Osc2Octave;

	// The semi-tones of oscillator 2. [-12.0, 12.0]
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset", meta = (ClampMin = "-12.0", ClampMax = "12.0", UIMin = "-12.0", UIMax = "12.0"))
	float Osc2Semitones;

	// The cents (hundreds of a semitone) of oscillator 2. [-100.0, 100.0]
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset", meta = (ClampMin = "-100.0", ClampMax = "100.0", UIMin = "-100.0", UIMax = "100.0"))
	float Osc2Cents;

	// The pulsewidth of oscillator 2 (when using a square wave type oscillator). [0.0, 1.0]
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float Osc2PulseWidth;

	// The amount of portamento to use, which is the amount of pitch sliding from current note to next [0.0, 1.0]
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float Portamento;

	// Enables forcing the oscillators to have no stereo spread.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset")
	uint32 bEnableUnison : 1;

	// Whether or not oscillator sync is enabled. Oscillator sync forces oscillator 2's phase to align with oscillator 1's phase.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset")
	uint32 bEnableOscillatorSync : 1;

	// The amount of stereo spread to use between oscillator 1 and oscillator 2 [0.0, 1.0]
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float Spread;

	// The stereo pan to use. 0.0 is center. -1.0 is left, 1.0 is right.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset", meta = (ClampMin = "-1.0", ClampMax = "1.0", UIMin = "-1.0", UIMax = "1.0"))
	float Pan;

	// The frequency to use for LFO 1 (in hz) [0.0, 50.0]
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset", meta = (ClampMin = "0.0", ClampMax = "50.0", UIMin = "0.0", UIMax = "50.0"))
	float LFO1Frequency;

	// The linear gain to use for LFO 1 [0.0, 1.0]
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float LFO1Gain;

	// The type of LFO to use for LFO 1
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset")
	ESynthLFOType LFO1Type;

	// The mode to use for LFO 1
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset")
	ESynthLFOMode LFO1Mode;

	// The built-in patch type to use for LFO 1 (you can route this to any patchable parameter using the Patches parameter)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset")
	ESynthLFOPatchType LFO1PatchType;

	// The frequency to use for LFO 2 (in hz) [0.0, 50.0]
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset", meta = (ClampMin = "0.0", ClampMax = "50.0", UIMin = "0.0", UIMax = "50.0"))
	float LFO2Frequency;

	// The linear gain to use for LFO 2 [0.0, 1.0]
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float LFO2Gain;

	// The type of LFO to use for LFO 2
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset")
	ESynthLFOType LFO2Type;

	// The mode to use for LFO 2
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset")
	ESynthLFOMode LFO2Mode;

	// The built-in patch type to use for LFO 2 (you can route this to any patchable parameter using the Patches parameter)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset")
	ESynthLFOPatchType LFO2PatchType;

	// The overall gain to use for the synthesizer in dB [-90.0, 20.0]
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset", meta = (DisplayName = "Gain (dB)", ClampMin = "-90.0", ClampMax = "20.0", UIMin = "-90.0", UIMax = "20.0"))
	float GainDb;

	// The amplitude envelope attack time (in ms) [0.0, 10000]
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset", meta = (ClampMin = "0.0", UIMin = "0.0", UIMax = "10000.0"))
	float AttackTime;

	// The amplitude envelope decay time (in ms)[0.0, 10000]
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset", meta = (ClampMin = "0.0", UIMin = "0.0", UIMax = "10000.0"))
	float DecayTime;

	// The amplitude envelope sustain amount (linear gain) [0.0, 1.0]
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float SustainGain;

	// The amplitude envelope release time (in ms) [0.0, 10000]
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset", meta = (ClampMin = "0.0", UIMin = "0.0", UIMax = "10000.0"))
	float ReleaseTime;

	// The built-in patch type for the envelope modulator
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset")
	ESynthModEnvPatch ModEnvPatchType;

	// The built-in patch type for the envelope modulator bias output. Bias is when the envelope output is offset by the sustain gain.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset")
	ESynthModEnvBiasPatch ModEnvBiasPatchType;

	// Whether or not to invert the modulation envelope
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset")
	uint32 bInvertModulationEnvelope : 1;

	// Whether or not to invert the modulation envelope bias output
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset")
	uint32 bInvertModulationEnvelopeBias : 1;

	// The "depth" (i.e. how much) modulation envelope to use. This scales the modulation envelope output. [0.0, 1.0]
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float ModulationEnvelopeDepth;

	// The modulation envelope attack time (in ms) [0.0, 10000]
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset", meta = (ClampMin = "0.0", ClampMax = "10000.0", UIMin = "0.0", UIMax = "10000.0"))
	float ModulationEnvelopeAttackTime;

	// The modulation envelope decay time (in ms) [0.0, 10000]
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset", meta = (ClampMin = "0.0", ClampMax = "10000.0", UIMin = "0.0", UIMax = "10000.0"))
	float ModulationEnvelopeDecayTime;

	// The modulation envelope sustain gain (linear gain) [0.0, 1.0]
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float ModulationEnvelopeSustainGain;

	// The modulation envelope release time (in ms) [0.0, 10000]
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset", meta = (ClampMin = "0.0", ClampMax = "10000.0", UIMin = "0.0", UIMax = "10000.0"))
	float ModulationEnvelopeReleaseTime;

	// Whether or not to use legato mode.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset")
	uint32 bLegato : 1;

	// Whether or not to use retrigger mode.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset")
	uint32 bRetrigger : 1;

	// The output filter cutoff frequency (hz) [0.0, 20000.0]
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset", meta = (ClampMin = "0.0", ClampMax = "20000.0", UIMin = "0.0", UIMax = "20000.0"))
	float FilterFrequency;

	// The output filter resonance (Q) [0.5, 10]
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset", meta = (ClampMin = "0.5", ClampMax = "10.0", UIMin = "0.5", UIMax = "10.0"))
	float FilterQ;

	// The output filter type (lowpass, highpass, bandpass, bandstop)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset")
	ESynthFilterType FilterType;

	// The output filter circuit/algorithm type (one-pole ladder, ladder, state-variable)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset")
	ESynthFilterAlgorithm FilterAlgorithm;

	// Whether or not stereo delay is enabled on the synth
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset")
	uint32 bStereoDelayEnabled : 1;

	// The stereo delay mode of the synth
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset")
	ESynthStereoDelayMode StereoDelayMode;

	// The stereo delay time (in ms) [0.0, 2000.0]
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset", meta = (ClampMin = "0.0", ClampMax = "2000.0", UIMin = "0.0", UIMax = "2000.0"))
	float StereoDelayTime;

	// The amount of feedback in the stereo delay line [0.0, 1.0]
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float StereoDelayFeedback;

	// The output wet level to use for the stereo delay time [0.0, 1.0]
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float StereoDelayWetlevel;

	// The ratio between left and right stereo delay lines (wider value is more separation) [0.0, 1.0]
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float StereoDelayRatio;

	// Whether or not the chorus effect is enabled
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset")
	uint32 bChorusEnabled : 1;

	// The depth of the chorus effect [0.0, 1.0]
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float ChorusDepth;

	// The amount of feedback in the chorus effect [0.0, 1.0]
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float ChorusFeedback;

	// The chorus LFO frequency [0.0, 20.0]
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset", meta = (ClampMin = "0.0", ClampMax = "20.0", UIMin = "0.0", UIMax = "20.0"))
	float ChorusFrequency;

	// The modular synth patch cords to use for the synth. Allows routing the LFO1/LFO2 and Modulation Envelope to any patchable destination. 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset")
	TArray<FEpicSynth1Patch> Patches;

	FModularSynthPreset()
		: bEnablePolyphony(false)
		, Osc1Type(ESynth1OscType::Saw)
		, Osc1Gain(1.0f)
		, Osc1Octave(0.0f)
		, Osc1Semitones(0.0f)
		, Osc1Cents(0.0f)
		, Osc1PulseWidth(0.5f)
		, Osc2Type(ESynth1OscType::Saw)
		, Osc2Gain(1.0f)
		, Osc2Octave(0.0f)
		, Osc2Semitones(0.0f)
		, Osc2Cents(2.5f)
		, Osc2PulseWidth(0.5f)
		, Portamento(0.0f)
		, bEnableUnison(false)
		, bEnableOscillatorSync(false)
		, Spread(0.5f)
		, Pan(0.0f)
		, LFO1Frequency(1.0f)
		, LFO1Gain(0.0f)
		, LFO1Type(ESynthLFOType::Sine)
		, LFO1Mode(ESynthLFOMode::Sync)
		, LFO1PatchType(ESynthLFOPatchType::PatchToNone)
		, LFO2Frequency(1.0f)
		, LFO2Gain(0.0f)
		, LFO2Type(ESynthLFOType::Sine)
		, LFO2Mode(ESynthLFOMode::Sync)
		, LFO2PatchType(ESynthLFOPatchType::PatchToNone)
		, GainDb(-3.0f)
		, AttackTime(10.0f)
		, DecayTime(100.0f)
		, SustainGain(0.707f)
		, ReleaseTime(5000.0f)
		, ModEnvPatchType(ESynthModEnvPatch::PatchToNone)
		, ModEnvBiasPatchType(ESynthModEnvBiasPatch::PatchToNone)
		, bInvertModulationEnvelope(false)
		, bInvertModulationEnvelopeBias(false)
		, ModulationEnvelopeDepth(1.0f)
		, ModulationEnvelopeAttackTime(10.0f)
		, ModulationEnvelopeDecayTime(100.0f)
		, ModulationEnvelopeSustainGain(0.707f)
		, ModulationEnvelopeReleaseTime(5000.0f)
		, bLegato(true)
		, bRetrigger(false)
		, FilterFrequency(8000.0f)
		, FilterQ(2.0f)
		, FilterType(ESynthFilterType::LowPass)
		, FilterAlgorithm(ESynthFilterAlgorithm::Ladder)
		, bStereoDelayEnabled(true)
		, StereoDelayMode(ESynthStereoDelayMode::PingPong)
		, StereoDelayTime(700.0f)
		, StereoDelayFeedback(0.7f)
		, StereoDelayWetlevel(0.3f)
		, StereoDelayRatio(0.2f)
		, bChorusEnabled(false)
		, ChorusDepth(0.2f)
		, ChorusFeedback(0.5f)
		, ChorusFrequency(2.0f)
	{}
};

USTRUCT(BlueprintType)
struct FModularSynthPresetBankEntry
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset")
	FString PresetName;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset")
	FModularSynthPreset Preset;

	FModularSynthPresetBankEntry()
		: PresetName(TEXT("Init"))
	{}
};


UCLASS(MinimalAPI, ClassGroup = Synth, meta = (BlueprintSpawnableComponent))
class UModularSynthPresetBank : public UObject
{
	GENERATED_BODY()

public:

 	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset")
 	TArray<FModularSynthPresetBankEntry> Presets;
};

UCLASS(MinimalAPI)
class UModularSynthLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

	// Adds the modular synth preset to the bank asset in the content browser. Only call during editor.
	UFUNCTION(BlueprintCallable, Category = "Synthesis")
	static void AddModularSynthPresetToBankAsset(UModularSynthPresetBank* InBank, const FModularSynthPreset& Preset, const FString& PresetName);
};


/**
* UModularSynthComponent
* Implementation of a USynthComponent.
*/
UCLASS(MinimalAPI, ClassGroup = Synth, meta = (BlueprintSpawnableComponent))
class UModularSynthComponent : public USynthComponent
{
	GENERATED_BODY()

	UModularSynthComponent(const FObjectInitializer& ObjectInitializer);

	// Initialize the synth component
	UE_API virtual bool Init(int32& SampleRate) override;

	// Called to generate more audio
	UE_API virtual int32 OnGenerateAudio(float* OutAudio, int32 NumSamples) override;

public:

	// The voice count to use for the synthesizer. Cannot be changed
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Synth|VoiceCount", meta = (ClampMin = "1", ClampMax = "32"))
	int32 VoiceCount;

	//// Define synth parameter functions

	// Play a new note. Optionally pass in a duration to automatically turn off the note.
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void NoteOn(const float Note, const int32 Velocity, const float Duration = -1.0f);

	// Stop the note (will only do anything if a voice is playing with that note)
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void NoteOff(const float Note, const bool bAllNotesOff = false, const bool bKillAllNotes = false);

	// Sets whether or not to use polyphony for the synthesizer.
	// @param bEnablePolyphony Whether or not to enable polyphony for the synth.
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetEnablePolyphony(bool bEnablePolyphony);

	// Set the oscillator gain. 
	// @param OscIndex Which oscillator to set the type for.
	// @param OscGain The oscillator gain.
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetOscGain(int32 OscIndex, float OscGain);

	// Set the oscillator gain modulation. 
	// @param OscIndex Which oscillator to set the type for.
	// @param OscGainMod The oscillator gain modulation to use.
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetOscGainMod(int32 OscIndex, float OscGainMod);

	// Set the oscillator frequency modulation
	// @param OscIndex Which oscillator to set the type for.
	// @param OscFreqMod The oscillator frequency modulation to use.
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetOscFrequencyMod(int32 OscIndex, float OscFreqMod);

	// Set the oscillator type. 
	// @param OscIndex Which oscillator to set the type for.
	// @param OscType The oscillator type to set.
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetOscType(int32 OscIndex, ESynth1OscType OscType);

	// Sets the oscillator octaves
	// @param OscIndex Which oscillator to set the type for.
	// @param Octave Which octave to set the oscillator to (relative to base frequency/pitch).
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetOscOctave(int32 OscIndex, float Octave);

	// Sets the oscillator semitones.
	// @param OscIndex Which oscillator to set the type for.
	// @param Semitones The amount of semitones to set the oscillator to (relative to base frequency/pitch).
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetOscSemitones(int32 OscIndex, float Semitones);

	// Sets the oscillator cents.
	// @param OscIndex Which oscillator to set the type for.
	// @param Cents The amount of cents to set the oscillator to (relative to base frequency/pitch)..
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetOscCents(int32 OscIndex, float Cents);

	// Sets the synth pitch bend amount.
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetPitchBend(float PitchBend);

	// Sets the synth portamento [0.0, 1.0]
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetPortamento(float Portamento);

	// Sets the square wave pulsewidth [0.0, 1.0]
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetOscPulsewidth(int32 OscIndex, float Pulsewidth);

	// Sets whether or not the synth is in unison mode (i.e. no spread)
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetEnableUnison(bool EnableUnison);

	// Set whether or not to follow the phase of osc2 to osc1
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetOscSync(const bool bIsSynced);

	// Sets the pan of the synth [-1.0, 1.0].
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetPan(float Pan);

	// Sets the amount of spread of the oscillators. [0.0, 1.0]
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetSpread(float Spread);

	// Sets the LFO frequency in hz
	// @param LFOIndex Which LFO to set the frequency for.
	// @param FrequencyHz The LFO frequency to use.
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetLFOFrequency(int32 LFOIndex, float FrequencyHz);

	// Sets the LFO frequency modulation in hz
	// @param LFOIndex Which LFO to set the frequency for.
	// @param FrequencyModHz The LFO frequency to use.
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetLFOFrequencyMod(int32 LFOIndex, float FrequencyModHz);

	// Sets the LFO gain factor
	// @param LFOIndex Which LFO to set the frequency for.
	// @param Gain The gain factor to use for the LFO.
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetLFOGain(int32 LFOIndex, float Gain);

	// Sets the LFO gain mod factor (external modulation)
	// @param LFOIndex Which LFO to set the frequency for.
	// @param Gain The gain factor to use for the LFO.
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetLFOGainMod(int32 LFOIndex, float GainMod);

	// Sets the LFO type
	// @param LFOIndex Which LFO to set the frequency for.
	// @param LFOType The LFO type to use.
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetLFOType(int32 LFOIndex, ESynthLFOType LFOType);

	// Sets the LFO type
	// @param LFOIndex Which LFO to set the frequency for.
	// @param LFOMode The LFO mode to use.
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetLFOMode(int32 LFOIndex, ESynthLFOMode LFOMode);

	// Sets the LFO patch type. LFO patch determines what parameter is modulated by the LFO.
	// @param LFOIndex Which LFO to set the frequency for.
	// @param LFOPatchType The LFO patch type to use.
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetLFOPatch(int32 LFOIndex, ESynthLFOPatchType LFOPatchType);

	// Sets the synth gain in decibels.
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio", meta = (DisplayName = "Set Gain (dB)"))
	UE_API void SetGainDb(float GainDb);

	// Sets the envelope attack time in msec.
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetAttackTime(float AttackTimeMsec);

	// Sets the envelope decay time in msec.
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetDecayTime(float DecayTimeMsec);

	// Sets the envelope sustain gain value.
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetSustainGain(float SustainGain);

	// Sets the envelope release time in msec.
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetReleaseTime(float ReleaseTimeMsec);

	// Sets whether or not to modulate a param based on the envelope. Uses bias envelope output (offset from sustain gain).
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetModEnvPatch(const ESynthModEnvPatch InPatchType);
	
	// Sets whether or not to modulate a param based on the envelope. Uses bias envelope output (offset from sustain gain).
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetModEnvBiasPatch(const ESynthModEnvBiasPatch InPatchType);

	// Sets whether or not to invert the envelope modulator.
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetModEnvInvert(const bool bInvert);

	// Sets whether or not to invert the bias output of the envelope modulator.
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetModEnvBiasInvert(const bool bInvert);

	// Sets the envelope modulator depth (amount to apply the output modulation)
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetModEnvDepth(const float Depth);

	// Sets the envelope modulator attack time in msec
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetModEnvAttackTime(const float AttackTimeMsec);

	// Sets the envelope modulator attack time in msec
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetModEnvDecayTime(const float DecayTimeMsec);

	// Sets the envelope modulator sustain gain
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetModEnvSustainGain(const float SustainGain);

	// Sets the envelope modulator release
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetModEnvReleaseTime(const float Release);

	// Sets whether or not to use legato for the synthesizer.
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetEnableLegato(bool LegatoEnabled);

	// Sets whether or not to retrigger envelope per note on.
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetEnableRetrigger(bool RetriggerEnabled);

	// Sets the filter cutoff frequency in hz.
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetFilterFrequency(float FilterFrequencyHz);

	// Sets the filter cutoff frequency in hz.
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetFilterFrequencyMod(float FilterFrequencyHz);

	// Sets the filter Q (resonance)
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetFilterQ(float FilterQ);

	// Sets a modulated filter Q (resonance)
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetFilterQMod(float FilterQ);

	// Sets the filter type.
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetFilterType(ESynthFilterType FilterType);

	// Sets the filter algorithm.
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetFilterAlgorithm(ESynthFilterAlgorithm FilterAlgorithm);

	// Sets whether not stereo delay is enabled.
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetStereoDelayIsEnabled(bool StereoDelayEnabled);

	// Sets whether not stereo delay is enabled.
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetStereoDelayMode(ESynthStereoDelayMode StereoDelayMode);

	// Sets the amount of stereo delay time in msec.
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetStereoDelayTime(float DelayTimeMsec);

	// Sets the amount of stereo delay feedback [0.0, 1.0]
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetStereoDelayFeedback(float DelayFeedback);

	// Sets the amount of stereo delay wetlevel [0.0, 1.0]
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetStereoDelayWetlevel(float DelayWetlevel);

	// Sets the amount of stereo delay ratio between left and right delay lines [0.0, 1.0]
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetStereoDelayRatio(float DelayRatio);

	// Sets whether or not chorus is enabled.
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetChorusEnabled(bool EnableChorus);

	// Sets the chorus depth
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetChorusDepth(float Depth);

	// Sets the chorus feedback
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetChorusFeedback(float Feedback);

	// Sets the chorus frequency
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetChorusFrequency(float Frequency);

	// Sets the preset struct for the synth
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetSynthPreset(const FModularSynthPreset& SynthPreset);

	// Creates a new modular synth patch between a modulation source and a set of modulation destinations
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API FPatchId CreatePatch(const ESynth1PatchSource PatchSource, const TArray<FSynth1PatchCable>& PatchCables, const bool bEnableByDefault);

	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API bool SetEnablePatch(const FPatchId PatchId, const bool bIsEnabled);

protected:
	Audio::FEpicSynth1 EpicSynth1;
};

#undef UE_API
