// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/SynthComponent.h"
#include "DSP/Granulator.h"
#include "Sound/SampleBufferIO.h"
#include "SynthComponentGranulator.generated.h"

#define UE_API SYNTHESIS_API


UENUM(BlueprintType)
enum class EGranularSynthEnvelopeType : uint8
{
	Rectangular,
	Triangle,
	DownwardTriangle,
	UpwardTriangle,
	ExponentialDecay,
	ExponentialIncrease,
	Gaussian,
	Hanning,
	Lanczos,
	Cosine,
	CosineSquared,
	Welch,
	Blackman,
	BlackmanHarris,
	Count UMETA(Hidden)
};

UENUM(BlueprintType)
enum class EGranularSynthSeekType : uint8
{
	FromBeginning,
	FromCurrentPosition,
	Count UMETA(Hidden)
};


UCLASS(MinimalAPI, ClassGroup = Synth, meta = (BlueprintSpawnableComponent))
class UGranularSynth : public USynthComponent
{
	GENERATED_BODY()

	UGranularSynth(const FObjectInitializer& ObjInitializer);
	~UGranularSynth();

	// Initialize the synth component
	UE_API virtual bool Init(int32& SampleRate) override;

	// Called to generate more audio
	UE_API virtual int32 OnGenerateAudio(float* OutAudio, int32 NumSamples) override;

	//~ Begin ActorComponent Interface.
	UE_API virtual void OnRegister() override;
	UE_API virtual void OnUnregister() override;
	UE_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction) override;
	//~ End ActorComponent Interface

public:

	// This will override the current sound wave if one is set, stop audio, and reload the new sound wave
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetSoundWave(USoundWave* InSoundWave);

	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetAttackTime(const float AttackTimeMsec);

	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetDecayTime(const float DecayTimeMsec);

	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetSustainGain(const float SustainGain);

	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetReleaseTimeMsec(const float ReleaseTimeMsec);

	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void NoteOn(const float Note, const int32 Velocity, const float Duration = -1.0f);

	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void NoteOff(const float Note, const bool bKill = false);

	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetGrainsPerSecond(const float InGrainsPerSecond);

	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetGrainProbability(const float InGrainProbability);

	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetGrainEnvelopeType(const EGranularSynthEnvelopeType EnvelopeType);

	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetPlaybackSpeed(const float InPlayheadRate);

	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetGrainPitch(const float BasePitch, const FVector2D PitchRange = FVector2D::ZeroVector);

	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetGrainVolume(const float BaseVolume, const FVector2D VolumeRange = FVector2D::ZeroVector);

	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetGrainPan(const float BasePan, const FVector2D PanRange = FVector2D::ZeroVector);

	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetGrainDuration(const float BaseDurationMsec, const FVector2D DurationRange = FVector2D::ZeroVector);

	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API float GetSampleDuration() const;

	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetScrubMode(const bool bScrubMode);

	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetPlayheadTime(const float InPositionSec, const float LerpTimeSec = 0.0f, EGranularSynthSeekType SeekType = EGranularSynthSeekType::FromBeginning);

	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API float GetCurrentPlayheadTime() const;

	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API bool IsLoaded() const;

protected:

	UPROPERTY(Transient)
	TObjectPtr<USoundWave> GranulatedSoundWave;

	Audio::FGranularSynth GranularSynth;
	Audio::FSoundWavePCMLoader SoundWaveLoader;

	bool bIsLoaded;
	bool bRegistered;

	bool bIsLoading;
};

#undef UE_API
