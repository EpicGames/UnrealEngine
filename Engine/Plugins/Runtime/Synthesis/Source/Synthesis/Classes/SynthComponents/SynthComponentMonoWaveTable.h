// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/SynthComponent.h"
#include "Components/SynthComponent.h"
#include "MonoWaveTable.h"
#include "SynthComponentMonoWaveTable.generated.h"

#define UE_API SYNTHESIS_API

enum class ESynthLFOType : uint8;


class USynthComponentMonoWaveTable;

// struct passed from Mono WaveTable Asset
// to USynthObject
struct AssetChangeInfo
{
	uint8 bNeedsFullRebuild : 1;
	int32 CurveThatWasAltered{ -1 };

	void FlagCurveAsAltered(const int32 CurveIndex)
	{
		if (CurveThatWasAltered == -1)
		{
			CurveThatWasAltered = CurveIndex;
		}
		else
		{
			bNeedsFullRebuild = true;
		}
	}

	AssetChangeInfo() : bNeedsFullRebuild(false) { }
}; // struct AssetChaneInfo

// UStruct Mono Wave Table Synth Preset
UCLASS(MinimalAPI, ClassGroup = Synth, meta = (BlueprintSpawnableComponent))
class UMonoWaveTableSynthPreset : public UObject
{
	GENERATED_BODY()

public:

	// ctor
	UE_API UMonoWaveTableSynthPreset();

	// Name the preset	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset")
	FString PresetName;

	// Lock wavetables to evenly spaced keyframes that can be edited vertically only (will re-sample)
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Synth|Preset", Meta = (InlineEditConditionToggle))
	uint8 bLockKeyframesToGridBool : 1;

	// How many evenly-spaced keyframes to use when LockKeyframesToGrid is true
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Synth|Preset", Meta = (EditCondition = "bLockKeyframesToGridBool", ClampMin = "3", UIMin = "3", ClampMax = "256", UIMax = "256"))
	int32 LockKeyframesToGrid;

	// How many samples will be taken of the curve from time = [0.0, 1.0]
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Synth|Preset", Meta = (ClampMin = "3", UIMin = "3", ClampMax = "4096", UIMax = "4096"))
	int32 WaveTableResolution;

	// Wave Table Editor
	UPROPERTY(EditAnywhere, NonTransactional, BlueprintReadOnly, Category = "Synth|Preset")
	TArray<FRuntimeFloatCurve> WaveTable;

	// Normalize the WaveTable data? False will allow clipping, True will normalize the tables when sent to the synth for rendering
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset")
	uint8 bNormalizeWaveTables : 1;

	// function to register lambdas to call when a property is changed
	UE_API void RegisterWTComponentCallback(uint32 ID, TFunction<void(const AssetChangeInfo& ChangeInfo)> Callback);

	// function to UNregister same lambdas
	UE_API void UnRegisterWTSynthComponentCallback(const uint32 ID);

protected:
#if WITH_EDITOR
	// Override on Post property change thing here (loop and call lambdas)
	UE_API virtual void PostEditChangeChainProperty(struct FPropertyChangedChainEvent&) override;
#endif // WITH_EDITOR

	UE_API void EditChangeInternal();

	UE_API void SampleAllToGrid(uint32 InGridsize);
	
	UE_API void SampleToGrid(uint32 InGridSize, uint32 InTableIndex);

	// Since wavetable synthesis sounds good when there are subtle changes between curves, 
	// this helps work flow when adding a new curve by making it a duplicate of a the curve before it
	UE_API void DuplicateCurveToEnd();

	// saves data from the asset to see what changed when
	UE_API void CacheAssetData();

	// Compares Underlying rich curves to see if the Index(th) curve was changed in the editor
	UE_API bool IsCachedTableEntryStillValid(Audio::DefaultWaveTableIndexType Index);

	// Map of registered TFunctions
	TMap<uint32, TFunction<void(const AssetChangeInfo&)>> PropertyChangedCallbacks;

	TArray<float> CurveBiDirTangents;

	// Default curve to use for work flow QOL (Never expose empty/silent curve)
	FRuntimeFloatCurve DefaultCurve;

	int32 CachedGridSize;

	// cached asset data (before last edit)
	int8 bWasLockedToGrid : 1;
	int32 CachedTableResolution;
	TArray<FRuntimeFloatCurve> CachedWaveTable;
	uint8 bCachedNormalizationSetting : 1;


	// let the USynth see into asset
	friend class USynthComponentMonoWaveTable;
};

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnTableAltered, int32, TableIndex);
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FNumTablesChanged);

UENUM(BlueprintType)
enum class CurveInterpolationType : uint8
{
	AUTOINTERP UMETA(DisplayName = "Auto"),
	LINEAR UMETA(DisplayName = "Linear"),
	CONSTANT UMETA(DisplayName = "Constant")
};


UCLASS(MinimalAPI, ClassGroup = Synth, meta = (BlueprintSpawnableComponent))
class USynthComponentMonoWaveTable : public USynthComponent
{
	GENERATED_BODY()

	UE_API USynthComponentMonoWaveTable(const FObjectInitializer& ObjectInitializer);

	// if in editor, unregister with the asset
	~USynthComponentMonoWaveTable()
	{
		if (CachedPreset)
		{
			CachedPreset->UnRegisterWTSynthComponentCallback(GetUniqueID());
		}
	}
	
	// Called when synth is created
	UE_API virtual bool Init(int32& InSampleRate) override;

	// Called to generate more audio
	UE_API virtual Audio::DefaultWaveTableIndexType OnGenerateAudio(float* OutAudio, int32 NumSamples) override;

public:
	/* Start BP functionality */

	// Get the number of table elements from Blueprint
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API int32 GetNumTableEntries();

	// Starts a new note (retrigs modulators, etc.)
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void NoteOn(const float InMidiNote, const float InVelocity);

	// Starts a new note (retrigs modulators, etc.)
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void NoteOff(const float InMidiNote);

	// Inform the synth if the sustain pedal is pressed or not
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetSustainPedalState(bool InSustainPedalState);

	// Sets the oscillator's frequency
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetFrequency(const float FrequencyHz);

	// Set a frequency offset in cents (for pitch modulation such as the Pitch Bend Wheel)
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetFrequencyPitchBend(const float FrequencyOffsetCents);

	// Set the oscillator's frequency via midi note number
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetFrequencyWithMidiNote(const float InMidiNote);

	// Sets the wavetable position. Expects a percentage between 0.0 and 1.0
	UFUNCTION(BLueprintCallable, Category = "Synth|Components|Audio", Meta = (ClampMin = "0.0", ClampMax = "1.0"))
	UE_API void SetWaveTablePosition(float InPosition);

	// Refresh a particular wavetable (from Game Thread data)
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void RefreshWaveTable(int32 Index);

	// Refresh all wavetables (from Game Thread data)
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void RefreshAllWaveTables();

	// Switch to another preset (STOPS SYNTH FROM PLAYING)
	//UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetSynthPreset(UMonoWaveTableSynthPreset* SynthPreset);

	// Set frequency of LFO controlling Table Position (in Hz)
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
		UE_API void SetPosLfoFrequency(const float InLfoFrequency);

	/* Set the Modulation depth of the Lfo controlling the Table Position around the current position value
	   0.0 = no modulation, 1.0 = current position +/- 0.5 (Lfo + Position result will clamp [0.0, 1.0]) */
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetPosLfoDepth(const float InLfoDepth);

	// Set the shape of the Lfo controlling the position
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetPosLfoType(const ESynthLFOType InLfoType);

	UE_API virtual void SetLowPassFilterFrequency(float InLowPassFilterFrequency) override;

	// Set the Cut-off frequency of the low-pass filter
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio",  Meta = (ClampMin = "0.0", ClampMax = "10.0"))
	UE_API void SetLowPassFilterResonance(float InNewQ);

	// Set Amp envelope attack time (msec)
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetAmpEnvelopeAttackTime(const float InAttackTimeMsec);
	
	// Set Amp envelope decay time (msec)
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetAmpEnvelopeDecayTime(const float InDecayTimeMsec);
	
	// Set Amp envelope sustain gain [0.0, 1.0]
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetAmpEnvelopeSustainGain(const float InSustainGain);
	
	// Set Amp envelope release time (msec)
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetAmpEnvelopeReleaseTime(const float InReleaseTimeMsec);
	
	// Set whether or not the Amp envelope is inverted
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetAmpEnvelopeInvert(const bool bInInvert);
	
	// Set whether or not the Amp envelope's bias is inverted
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetAmpEnvelopeBiasInvert(const bool bInBiasInvert);
	
	// Set the overall depth of the Amp envelope
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetAmpEnvelopeDepth(const float InDepth);
	
	// Set the bias depth of the the Amp envelope
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetAmpEnvelopeBiasDepth(const float InDepth);

	// Set Low-Pass Filter envelope attack time (msec)
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetFilterEnvelopeAttackTime(const float InAttackTimeMsec);
	
	// Set Low-Pass Filter envelope decay time (msec)
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetFilterEnvelopenDecayTime(const float InDecayTimeMsec);
	
	// Set Low-Pass Filter envelope sustain gain
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetFilterEnvelopeSustainGain(const float InSustainGain);
	
	// Set Low-Pass Filter envelope release time (msec)
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetFilterEnvelopeReleaseTime(const float InReleaseTimeMsec);
	
	// Set Low-Pass Filter envelope inversion
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetFilterEnvelopeInvert(const bool bInInvert);
	
	// Set Low-Pass Filter envelope bias inversion
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetFilterEnvelopeBiasInvert(const bool bInBiasInvert);
	
	// Set Low-Pass Filter envelope depth
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetFilterEnvelopeDepth(const float InDepth);
	
	// Set Low-Pass Filter envelope bias depth
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetFilterEnvelopeBiasDepth(const float InDepth);

	// Set Position envelope attack time (msec)
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetPositionEnvelopeAttackTime(const float InAttackTimeMsec);

	// Set Position envelope decay time (msec)
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetPositionEnvelopeDecayTime(const float InDecayTimeMsec);

	// Set Position envelope sustain gain
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetPositionEnvelopeSustainGain(const float InSustainGain);

	// Set Position envelope release time (msec)
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetPositionEnvelopeReleaseTime(const float InReleaseTimeMsec);

	// Set Position envelope envelope inversion
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetPositionEnvelopeInvert(const bool bInInvert);

	// Set Position envelope bias inversion
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetPositionEnvelopeBiasInvert(const bool bInBiasInvert);

	// Set Position envelope envelope depth
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetPositionEnvelopeDepth(const float InDepth);

	// Set Position envelope bias depth
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetPositionEnvelopeBiasDepth(const float InDepth);

	// Get the number of curves in the wave table. (returns -1 if there is no asset)
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	int32 GetMaxTableIndex() const { return CachedPreset? CachedPreset->WaveTable.Num() - 1 : -1; }

	/* Set a Keyframe value given a Table number and Keyframe number.
	   Returns false if the request was invalid.
	   NewValue will be clamped from +/- 1.0 */
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API bool SetCurveValue(int32 TableIndex, int32 KeyframeIndex, const float NewValue);

	/*
		Set the curve interpolation type (What the curve is doing between keyframes)
		This should only be used for live-editing features! (changing the curves at runtime is expensive)
	*/
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API bool SetCurveInterpolationType(CurveInterpolationType InterpolationType,  int32 TableIndex);

	/*
		Set the curve tangent ("Curve depth" between keyframes)
		This should only be used for live-editing features! (changing the curves at runtime is expensive)
	*/
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API bool SetCurveTangent(int32 TableIndex, float InNewTangent);

	// TODO: Enable this functionality when Curve bug is fixed
/*
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	bool AddTableToEnd();

	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	bool RemoveTableFromEnd();
*/

// Get the curve interpolation type (What the curve is doing between keyframes)
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API float GetCurveTangent(int32 TableIndex);

	// Get an array of floats that represent the key frames in the requested curve
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API TArray<float> GetKeyFrameValuesForTable(float TableIndex) const;

	// Wave Table curve was edited
	UPROPERTY(BlueprintAssignable, Category = "Synth|Preset")
	FOnTableAltered OnTableAltered;

	// Curve array altered
	UPROPERTY(BlueprintAssignable, Category = "Synth|Preset")
	FNumTablesChanged OnNumTablesChanged;

protected:
	// Helper function that resets a curve to a default saw-tooth
	// (so a curve is always immediately audible without user effort)
	UE_API void ResetCurve(int32 Index);

	// Initializes the underlying synthesizer.
	// Called when underlying wave table containers in the Synth FObject need resizing.
	UE_API void InitSynth();

	// Callback for the UAsset this synth component is subscribed to
	// Compares cached data to current UAsset data and makes updates
	UE_API void ReactToAssetChange(const AssetChangeInfo& ChangeInfo);

#if WITH_EDITOR
	// Override on Post property change thing here (see if we got a new preset)
	UE_API virtual void PostEditChangeChainProperty(struct FPropertyChangedChainEvent& Eevnt) override;
#endif // WITH_EDITOR

	/** The settings asset to use for this synth */
	UPROPERTY(EditAnywhere, Category = "Synth|Components|Audio")
	TObjectPtr<UMonoWaveTableSynthPreset> CurrentPreset;

	UMonoWaveTableSynthPreset* CachedPreset;

	// underlying wavetable synth
	Audio::FMonoWaveTable Synth;
	Audio::DefaultWaveTableIndexType SampleRate;
};

#undef UE_API
