// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/SynthComponent.h"
#include "DSP/SinOsc.h"
#include "DSP/AudioBufferDistanceAttenuation.h"


#include "SynthComponentToneGenerator.generated.h"

#define UE_API SYNTHESIS_API


class FToneGenerator : public ISoundGenerator
{
public:
	FToneGenerator(int32 InSampleRate, int32 InNumChannels, int32 InFrequency, float InVolume, const Audio::FAudioBufferDistanceAttenuationSettings& InAttenuationSettings);
	virtual ~FToneGenerator() = default;

	//~ Begin FSoundGenerator 
	virtual int32 GetNumChannels() const override { return NumChannels; };
	virtual int32 OnGenerateAudio(float* OutAudio, int32 NumSamples) override;
	//~ End FSoundGenerator

	void SetFrequency(float InFrequency);
	void SetVolume(float InVolume);
	void SetDistance(float InDistance);

private:
	int32 NumChannels = 2;
	Audio::FSineOsc SineOsc;
	Audio::FAlignedFloatBuffer Buffer;

	float CurrentDistance = 0.0f;
	float CurrentAttenuation = 1.0f;
	Audio::FAudioBufferDistanceAttenuationSettings DistanceAttenuationSettings;
};

UCLASS(MinimalAPI, ClassGroup = Synth, meta = (BlueprintSpawnableComponent))
class USynthComponentToneGenerator : public USynthComponent
{
	GENERATED_BODY()

	USynthComponentToneGenerator(const FObjectInitializer& ObjInitializer);
	UE_API virtual ~USynthComponentToneGenerator();

public:
	// The frequency (in hz) of the tone generator.
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Tone Generator", meta = (ClampMin = "10.0", ClampMax = "20000.0"))
	float Frequency;

	// The linear volume of the tone generator.
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Tone Generator", meta = (ClampMin = "0.0", ClampMax = "1.0"))
	float Volume;

	/* A distance attenuation curve to use to attenuate the audio. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attenuation)
	FRuntimeFloatCurve DistanceAttenuationCurve;

	/* A distance range over which to apply distance attenuation using the supplied curve. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attenuation)
	FVector2D DistanceRange = { 400.0f, 4000.0f };

	/* An attenuation, in decibels, to apply to the sound at max range. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attenuation)
	float AttenuationDbAtMaxRange = -60.0f;

	// Sets the frequency of the tone generator
	UFUNCTION(BlueprintCallable, Category = "Tone Generator")
	UE_API void SetFrequency(float InFrequency);

	// Sets the volume of the tone generator
	UFUNCTION(BlueprintCallable, Category = "Tone Generator")
	UE_API void SetVolume(float InVolume);

	// Tick this component to get the current position of the component to pass to the attenuator
	UE_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;

	UE_API virtual ISoundGeneratorPtr CreateSoundGenerator(const FSoundGeneratorInitParams& InParams) override;

public:

private:
	Audio::FAudioBufferDistanceAttenuationSettings DistanceAttenuationSettings;
	ISoundGeneratorPtr ToneGenerator;
};

#undef UE_API
