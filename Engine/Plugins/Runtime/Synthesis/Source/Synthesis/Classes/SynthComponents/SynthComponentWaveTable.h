// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/SynthComponent.h"
#include "DSP/SampleBufferReader.h"
#include "Sound/SampleBufferIO.h"
#include "SynthComponentWaveTable.generated.h"

#define UE_API SYNTHESIS_API

UENUM(BlueprintType)
enum class ESamplePlayerSeekType : uint8
{
	FromBeginning,
	FromCurrentPosition,
	FromEnd,
	Count UMETA(Hidden)
};


// Called when a sample has finished loading into the sample player
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnSampleLoaded);

// Called while a sample player is playing back. Indicates the playhead progress in percent and as absolute time value (within the file).
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnSamplePlaybackProgress, float, ProgressPercent, float, ProgressTimeSeconds);


UCLASS(MinimalAPI, ClassGroup = Synth, meta = (BlueprintSpawnableComponent))
class USynthSamplePlayer : public USynthComponent
{
	GENERATED_BODY()

	USynthSamplePlayer(const FObjectInitializer& ObjInitializer);
	~USynthSamplePlayer();

	// Initialize the synth component
	UE_API virtual bool Init(int32& SampleRate) override;

	// Called to generate more audio
	UE_API virtual int32 OnGenerateAudio(float* OutAudio, int32 NumSamples) override;

	//~ Begin ActorComponent Interface.
	UE_API virtual void OnRegister() override;
	UE_API virtual void OnUnregister() override;
	UE_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction) override;
	//~ End ActorComponent Interface

public:

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset")
	TObjectPtr<USoundWave> SoundWave;

	UPROPERTY(BlueprintAssignable, Category = "Synth|Components|Audio")
	FOnSampleLoaded OnSampleLoaded;

	UPROPERTY(BlueprintAssignable, Category = "Synth|Components|Audio")
	FOnSamplePlaybackProgress OnSamplePlaybackProgress;

	// This will override the current sound wave if one is set, stop audio, and reload the new sound wave
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetSoundWave(USoundWave* InSoundWave);

	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetPitch(float InPitch, float TimeSec);

	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SeekToTime(float TimeSec, ESamplePlayerSeekType SeekType, bool bWrap = true);

	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetScrubMode(bool bScrubMode);

	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API void SetScrubTimeWidth(float InScrubTimeWidthSec);

	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API float GetSampleDuration() const;

	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API float GetCurrentPlaybackProgressTime() const;

	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API float GetCurrentPlaybackProgressPercent() const;

	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	UE_API bool IsLoaded() const;

protected:
	UE_API void LoadSoundWaveInternal();

	Audio::FSampleBufferReader SampleBufferReader;
	Audio::TSampleBuffer<int16> SampleBuffer;
	Audio::FSoundWavePCMLoader SoundWaveLoader;

	float SampleDurationSec;
	float SamplePlaybackProgressSec;

	bool bIsLoaded;
};

#undef UE_API
