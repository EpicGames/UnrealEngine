// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConvolutionReverb.h"

#include "DSP/AlignedBlockBuffer.h"
#include "DSP/FloatArrayMath.h"
#include "DSP/SampleRateConverter.h"
#include "SubmixChannelFormatConverter.h"
#include "SynthesisModule.h"

namespace Audio
{
	namespace ConvolutionReverbIntrinsics
	{
		// Reverb output formats by # of channels.
		static const TArray<TArray<EAudioMixerChannel::Type>> ReverbOutputFormats = 
		{
			// Zero channels (none)
			{},

			// One channel (mono)
			{ EAudioMixerChannel::FrontCenter },

			// Two channels (stereo)
			{ EAudioMixerChannel::FrontLeft, EAudioMixerChannel::FrontRight },

			// Three channels (fronts)
			{ EAudioMixerChannel::FrontLeft, EAudioMixerChannel::FrontCenter, EAudioMixerChannel::FrontRight },

			// Four channels (quad)
			{
				EAudioMixerChannel::Type::FrontLeft,
				EAudioMixerChannel::Type::FrontRight,
				EAudioMixerChannel::Type::SideLeft,
				EAudioMixerChannel::Type::SideRight
			},

			// Five channels (5.0)
			{
				EAudioMixerChannel::Type::FrontLeft,
				EAudioMixerChannel::Type::FrontRight,
				EAudioMixerChannel::Type::FrontCenter,
				EAudioMixerChannel::Type::SideLeft,
				EAudioMixerChannel::Type::SideRight
			},

			// Six channels (5.1)
			{
				EAudioMixerChannel::Type::FrontLeft,
				EAudioMixerChannel::Type::FrontRight,
				EAudioMixerChannel::Type::FrontCenter,
				EAudioMixerChannel::Type::LowFrequency,
				EAudioMixerChannel::Type::SideLeft,
				EAudioMixerChannel::Type::SideRight
			},

			// Seven channels (7.0)
			{
				EAudioMixerChannel::Type::FrontLeft,
				EAudioMixerChannel::Type::FrontRight,
				EAudioMixerChannel::Type::FrontCenter,
				EAudioMixerChannel::Type::BackLeft,
				EAudioMixerChannel::Type::BackRight,
				EAudioMixerChannel::Type::SideLeft,
				EAudioMixerChannel::Type::SideRight
			},

			// Eight channels (7.1)
			{
				EAudioMixerChannel::Type::FrontLeft,
				EAudioMixerChannel::Type::FrontRight,
				EAudioMixerChannel::Type::FrontCenter,
				EAudioMixerChannel::Type::LowFrequency,
				EAudioMixerChannel::Type::BackLeft,
				EAudioMixerChannel::Type::BackRight,
				EAudioMixerChannel::Type::SideLeft,
				EAudioMixerChannel::Type::SideRight
			}
		};

		bool GetReverbOutputChannelTypesForNumChannels(int32 InNumChannels, TArray<EAudioMixerChannel::Type>& OutChannelTypes)
		{
			OutChannelTypes.Reset();

			if ((InNumChannels >= 0) && (InNumChannels < ReverbOutputFormats.Num()))
			{
				OutChannelTypes = ReverbOutputFormats[InNumChannels];

				return true;
			}

			return false;
		}

		// Sets impulse response on convolution algorithm. Handles resampling and deinterleaving.
		bool SetImpulseResponse(IConvolutionAlgorithm& InAlgo, const TArray<float>& InSamples, int32 InNumImpulseResponses, float InImpulseSampleRate, float InTargetSampleRate)
		{
			const TArray<float>* TargetImpulseSamples = &InSamples;

			TArray<float> ResampledImpulseSamples;

			// Prepare impulse samples by converting samplerate and deinterleaving.
			if (InImpulseSampleRate != InTargetSampleRate)
			{
				// convert sample rate of impulse 
				float SampleRateRatio = InImpulseSampleRate / InTargetSampleRate;

				TUniquePtr<ISampleRateConverter> Converter(ISampleRateConverter::CreateSampleRateConverter());

				if (!Converter.IsValid())
				{
					UE_LOGF(LogSynthesis, Error, "ISampleRateConverter failed to create a sample rate converter");
					return false;
				}

				Converter->Init(SampleRateRatio, InNumImpulseResponses);
				Converter->ProcessFullbuffer(InSamples.GetData(), InSamples.Num(), ResampledImpulseSamples);

				TargetImpulseSamples = &ResampledImpulseSamples;
			}

			const int32 NumFrames = TargetImpulseSamples->Num() / InNumImpulseResponses;

			// Prepare deinterleave pointers
			TArray<FAlignedFloatBuffer> DeinterleaveSamples;
			while (DeinterleaveSamples.Num() < InNumImpulseResponses)
			{
				FAlignedFloatBuffer& Buffer = DeinterleaveSamples.Emplace_GetRef();
				if (NumFrames > 0)
				{
					Buffer.AddUninitialized(NumFrames);
				}
			}

			// Deinterleave impulse samples
			FConvolutionReverb::DeinterleaveBuffer(DeinterleaveSamples, *TargetImpulseSamples, InNumImpulseResponses);

			// Set impulse responses in algorithm
			for (int32 i = 0; i < DeinterleaveSamples.Num(); i++)
			{
				const FAlignedFloatBuffer& Buffer = DeinterleaveSamples[i];

				InAlgo.SetImpulseResponse(i, Buffer.GetData(), Buffer.Num());
			}

			return true;
		}
	}

	FConvolutionReverb::FChannelFormatConverterWrapper::FChannelFormatConverterWrapper(TUniquePtr<FSimpleUpmixer> InConverter)
	:	Storage(nullptr)
	,	BaseConverter(nullptr)
	,	SimpleUpmixer(nullptr)
	{
		BaseConverter = static_cast<IChannelFormatConverter*>(InConverter.Get());
		SimpleUpmixer = InConverter.Get();
		Storage = MoveTemp(InConverter);
	}

	FConvolutionReverb::FChannelFormatConverterWrapper::FChannelFormatConverterWrapper(TUniquePtr<IChannelFormatConverter> InConverter)
	:	Storage(nullptr)
	,	BaseConverter(nullptr)
	,	SimpleUpmixer(nullptr)
	{
		BaseConverter = static_cast<IChannelFormatConverter*>(InConverter.Get());
		Storage = MoveTemp(InConverter);
	}

	FConvolutionReverb::FChannelFormatConverterWrapper::FChannelFormatConverterWrapper(FChannelFormatConverterWrapper&& InOther)
	{
		*this = MoveTemp(InOther);
	}

	FConvolutionReverb::FChannelFormatConverterWrapper& FConvolutionReverb::FChannelFormatConverterWrapper::operator=(FChannelFormatConverterWrapper&& InOther)
	{
		Storage = MoveTemp(InOther.Storage);
		BaseConverter = InOther.BaseConverter;
		SimpleUpmixer = InOther.SimpleUpmixer;

		InOther.BaseConverter = nullptr;
		InOther.SimpleUpmixer = nullptr;

		return *this;
	}

	bool FConvolutionReverb::FChannelFormatConverterWrapper::IsValid() const
	{
		return (Storage.IsValid() && (nullptr != BaseConverter));
	}

	const FConvolutionReverb::FInputFormat& FConvolutionReverb::FChannelFormatConverterWrapper::GetInputFormat() const
	{
		if (nullptr != BaseConverter)
		{
			return BaseConverter->GetInputFormat();
		}
		return DefaultInputFormat;
	}

	const FConvolutionReverb::FOutputFormat& FConvolutionReverb::FChannelFormatConverterWrapper::GetOutputFormat() const
	{
		if (nullptr != BaseConverter)
		{
			return BaseConverter->GetOutputFormat();
		}
		return DefaultOutputFormat;
	}
	
	void FConvolutionReverb::FChannelFormatConverterWrapper::ProcessAudio(const TArray<FAlignedFloatBuffer>& InInputBuffers, TArray<FAlignedFloatBuffer>& OutOutputBuffers)
	{
		if (nullptr != BaseConverter)
		{
			BaseConverter->ProcessAudio(InInputBuffers, OutOutputBuffers);
		}
	}

	void FConvolutionReverb::FChannelFormatConverterWrapper::SetRearChannelBleed(float InGain, bool bFadeToGain)
	{
		if (nullptr != SimpleUpmixer)
		{
			SimpleUpmixer->SetRearChannelBleed(InGain, bFadeToGain);
		}
	}

	void FConvolutionReverb::FChannelFormatConverterWrapper::SetRearChannelFlip(bool bInDoRearChannelFlip, bool bFadeFlip)
	{
		if (nullptr != SimpleUpmixer)
		{
			SimpleUpmixer->SetRearChannelFlip(bInDoRearChannelFlip, bFadeFlip);
		}
	}

	bool FConvolutionReverb::FChannelFormatConverterWrapper::GetRearChannelFlip() const
	{
		if (nullptr != SimpleUpmixer)
		{
			return SimpleUpmixer->GetRearChannelFlip();
		}

		return false;
	}

	FConvolutionReverb::FInputFormat FConvolutionReverb::FChannelFormatConverterWrapper::DefaultInputFormat;
	FConvolutionReverb::FOutputFormat FConvolutionReverb::FChannelFormatConverterWrapper::DefaultOutputFormat;

	const FConvolutionReverbSettings FConvolutionReverbSettings::DefaultSettings;
	FConvolutionReverbSettings::FConvolutionReverbSettings()
	:	NormalizationVolume(-24.f)
	,	RearChannelBleed(0.f)
	,	bRearChannelFlip(false)
	{}

	// Create a convolution algorithm. This performs creation of the convolution algorithm object,
	// converting sample rates of impulse responses, sets the impulse response and initializes the
	// gain matrix of the convolution algorithm.
	//
	// @params InInitData - Contains all the information needed to create a convolution algorithm.
	//
	// @return TUniquePtr<IConvolutionAlgorithm>  Will be invalid if there was an error.
	TUniquePtr<FConvolutionReverb> FConvolutionReverb::CreateConvolutionReverb(const FConvolutionReverbInitData& InInitData, const FConvolutionReverbSettings& InSettings)
	{
		using namespace ConvolutionReverbIntrinsics;

		// Check valid sample rates
		if (InInitData.ImpulseSampleRate <= 0.f)
		{
			UE_LOGF(LogSynthesis, Error, "Invalid impulse sample rate: %f.", InInitData.ImpulseSampleRate);
			return TUniquePtr<FConvolutionReverb>();
		}

		if (InInitData.TargetSampleRate <= 0.f)
		{
			UE_LOGF(LogSynthesis, Error, "Invalid target sample rate: %f.", InInitData.TargetSampleRate);
			return TUniquePtr<FConvolutionReverb>();
		}

		const FConvolutionSettings& AlgoSettings = InInitData.AlgorithmSettings;

		// Check valid channel counts
		if (AlgoSettings.NumInputChannels < 1)
		{
			UE_LOGF(LogSynthesis, Error, "Invalid num input channels: %d", AlgoSettings.NumInputChannels);
			return TUniquePtr<FConvolutionReverb>();
		}
		
		if (AlgoSettings.NumOutputChannels < 1)
		{
			UE_LOGF(LogSynthesis, Error, "Invalid num output channels: %d", AlgoSettings.NumOutputChannels);
			return TUniquePtr<FConvolutionReverb>();
		}

		if (AlgoSettings.NumImpulseResponses < 1)
		{
			UE_LOGF(LogSynthesis, Error, "Invalid num impulse responses: %d", AlgoSettings.NumImpulseResponses);
			return TUniquePtr<FConvolutionReverb>();
		}

		// Check input audio format 
		const FInputFormat& InputAudioFormat = InInitData.InputAudioFormat;

		const bool bValidInputFormat = (InputAudioFormat.NumChannels > 0);

		if (!bValidInputFormat)
		{
			UE_LOGF(LogSynthesis, Error, "Invalid input audio format. Channel count: %d", InputAudioFormat.NumChannels);
			return TUniquePtr<FConvolutionReverb>();
		}

		const bool bMatchingInputFormat = InInitData.bMixInputChannelFormatToImpulseResponseFormat || 
			(InputAudioFormat.NumChannels == AlgoSettings.NumInputChannels);

		if (!bMatchingInputFormat)
		{
			UE_LOGF(LogSynthesis, Error, "Mismatched input format. Received %d input channels. Expected %d.", InputAudioFormat.NumChannels, AlgoSettings.NumInputChannels);
			return TUniquePtr<FConvolutionReverb>();
		}

		// Check input audio format 
		const FOutputFormat& OutputAudioFormat = InInitData.OutputAudioFormat;

		const bool bValidOutputFormat = (OutputAudioFormat.NumChannels > 0);

		if (!bValidOutputFormat)
		{
			UE_LOGF(LogSynthesis, Error, "Invalid output audio format. Channel count: %d", OutputAudioFormat.NumChannels);
			return TUniquePtr<FConvolutionReverb>();
		}

		// Create convolution algorithm
		TUniquePtr<IConvolutionAlgorithm> ConvolutionAlgorithm = FConvolutionFactory::NewConvolutionAlgorithm(AlgoSettings);

		if (!ConvolutionAlgorithm.IsValid())
		{
			UE_LOGF(LogSynthesis, Warning, "Failed to create convolution algorithm for convolution reverb.");
			return TUniquePtr<FConvolutionReverb>();
		}

		// Set impulse responses onto convolution algorithm object.
		bool bSuccess = SetImpulseResponse(*ConvolutionAlgorithm, InInitData.Samples, InInitData.AlgorithmSettings.NumImpulseResponses, InInitData.ImpulseSampleRate, InInitData.TargetSampleRate);

		if (!bSuccess)
		{
			UE_LOGF(LogSynthesis, Warning, "Failed to set impulse responses on convolution algorithm.");
			return TUniquePtr<FConvolutionReverb>();
		}

		// Set gain matrix on convolution algorithm object.
		for (const FConvolutionReverbGainEntry& Entry : InInitData.GainMatrix)
		{
			ConvolutionAlgorithm->SetMatrixGain(Entry.InputIndex, Entry.ImpulseIndex, Entry.OutputIndex, Entry.Gain);
		}

		TUniquePtr<IChannelFormatConverter> InputConverter;

		// Create input audio channel format converter for input audio.
		if (InInitData.bMixInputChannelFormatToImpulseResponseFormat && (InputAudioFormat.NumChannels != AlgoSettings.NumInputChannels))
		{
			FOutputFormat AlgoFormat;
			AlgoFormat.NumChannels = AlgoSettings.NumInputChannels;

			InputConverter = FAC3DownmixerFactory::CreateAC3Downmixer(InputAudioFormat, AlgoFormat, AlgoSettings.BlockNumSamples);

			if (!InputConverter.IsValid())
			{
				UE_LOGF(LogSynthesis, Warning, "Failed to convert input audio format to convolution reverb input format.");
				return TUniquePtr<FConvolutionReverb>();
			}
		}

		FChannelFormatConverterWrapper OutputConverter;


		// Create output audio channel format conveter for output audio.
		const bool bDownmixReverbToOutput = InInitData.bMixReverbOutputToOutputChannelFormat && (AlgoSettings.NumOutputChannels > OutputAudioFormat.NumChannels);
		const bool bUpmixReverbToOutput = InInitData.bMixReverbOutputToOutputChannelFormat && (AlgoSettings.NumOutputChannels < OutputAudioFormat.NumChannels);
		const bool bRouteReverbToOutput = !bDownmixReverbToOutput && !bUpmixReverbToOutput && (OutputAudioFormat.NumChannels != AlgoSettings.NumOutputChannels);

		if (bDownmixReverbToOutput)
		{
			FInputFormat AlgoFormat;
			AlgoFormat.NumChannels = AlgoSettings.NumOutputChannels;
			OutputConverter = FChannelFormatConverterWrapper(FAC3DownmixerFactory::CreateAC3Downmixer(AlgoFormat, OutputAudioFormat, AlgoSettings.BlockNumSamples));
		}
		else if (bUpmixReverbToOutput || bRouteReverbToOutput)
		{
			// Get channel info for reverb output
			TArray<EAudioMixerChannel::Type> ReverbChannelTypes;
			if (!GetReverbOutputChannelTypesForNumChannels(AlgoSettings.NumOutputChannels, ReverbChannelTypes))
			{
				UE_LOGF(LogSynthesis, Warning, "Failed to handle reverb output channel count [%d]", AlgoSettings.NumOutputChannels);
				return TUniquePtr<FConvolutionReverb>();
			}

			// Get channel info for output audio
			TArray<EAudioMixerChannel::Type> OutputAudioChannelTypes;
			if (!GetSubmixChannelOrderForNumChannels(OutputAudioFormat.NumChannels, OutputAudioChannelTypes))
			{
				UE_LOGF(LogSynthesis, Warning, "Failed to handle output audio channel count [%d]", OutputAudioFormat.NumChannels);
				return TUniquePtr<FConvolutionReverb>();
			}

			if (bUpmixReverbToOutput)
			{
				OutputConverter = FChannelFormatConverterWrapper(FSimpleUpmixer::CreateSimpleUpmixer(ReverbChannelTypes, OutputAudioChannelTypes, AlgoSettings.BlockNumSamples));
			}
			else if (bRouteReverbToOutput)
			{
				OutputConverter = FChannelFormatConverterWrapper(FSimpleRouter::CreateSimpleRouter(ReverbChannelTypes, OutputAudioChannelTypes, AlgoSettings.BlockNumSamples));
			}
		}

		// Override IR normalization in settings with InitData normalization
		FConvolutionReverbSettings UpdatedSettings = InSettings;
		UpdatedSettings.NormalizationVolume = InInitData.NormalizationVolume;

		return TUniquePtr<FConvolutionReverb>(new FConvolutionReverb(MoveTemp(ConvolutionAlgorithm), MoveTemp(InputConverter), MoveTemp(OutputConverter), UpdatedSettings));
	}


	FConvolutionReverb::FConvolutionReverb(TUniquePtr<IConvolutionAlgorithm> InAlgorithm, TUniquePtr<IChannelFormatConverter> InInputConverter, FChannelFormatConverterWrapper&& InOutputConverter, const FConvolutionReverbSettings& InSettings)
	:	Settings(InSettings)
	,	ConvolutionAlgorithm(MoveTemp(InAlgorithm))
	,	InputChannelFormatConverter(MoveTemp(InInputConverter))
	,	OutputChannelFormatConverter(MoveTemp(InOutputConverter))
	,	ExpectedNumFramesPerCallback(256)
	,	NumInputSamplesPerBlock(0)
	,	NumOutputSamplesPerBlock(0)
	,	OutputGain(1.f)
	,	bIsConvertingInputChannelFormat(false)
	,	bIsConvertingOutputChannelFormat(false)
	{
		bIsConvertingInputChannelFormat = InputChannelFormatConverter.IsValid();
		bIsConvertingOutputChannelFormat = OutputChannelFormatConverter.IsValid();

		ResizeProcessingBuffers();
		ResizeBlockBuffers();

		const bool bFadeUpdate = false;
		Update(bFadeUpdate);

	}

	FConvolutionReverb::~FConvolutionReverb() = default;

	void FConvolutionReverb::SetSettings(const FConvolutionReverbSettings& InSettings)
	{
		Settings = InSettings;

		const bool bFadeUpdate = true;
		Update(bFadeUpdate);
	}

	const FConvolutionReverbSettings& FConvolutionReverb::GetSettings() const
	{
		return Settings;
	}

	void FConvolutionReverb::ProcessAudio(int32 InNumInputChannels, const FAlignedFloatBuffer& InputAudio, int32 InNumOutputChannels, FAlignedFloatBuffer& OutputAudio)
	{
		check(InNumInputChannels != 0);
		check(InputAudio.Num() % InNumInputChannels == 0);

		ProcessAudio(InNumInputChannels, InputAudio.GetData(), InNumOutputChannels, OutputAudio.GetData(), InputAudio.Num() / InNumInputChannels);
	}

	void FConvolutionReverb::ProcessAudio(int32 InNumInputChannels, const float* InputAudio, int32 InNumOutputChannels, float* OutputAudio, const int32 InNumFrames)
	{
		check(InputBlockBuffer.IsValid());
		check(OutputBlockBuffer.IsValid());

		if (InNumFrames < 1)
		{
			return;
		}

		const int32 NumInputSamples = InNumFrames * InNumInputChannels;
		const int32 NumOutputSamples = InNumFrames * InNumOutputChannels;

		if (InNumFrames != ExpectedNumFramesPerCallback)
		{
			ExpectedNumFramesPerCallback = InNumFrames;
			// If the number of frames processed per a call changes, then
			// block buffers need to be updated. 
			ResizeBlockBuffers();
		}

		if (NumOutputSamples > 0)
		{
			FMemory::Memzero(OutputAudio, NumOutputSamples * sizeof(float));
		}

		// If we don't have a valid algorithm. We cannot do any processing. 
		if (!ConvolutionAlgorithm.IsValid())
		{
			// Early exit due to invalid algorithm pointer
			return;
		}

		// Our convolution algorithm should have equal number of input and output channels.
		if (InNumInputChannels != GetNumInputChannels())
		{
			UE_LOGF(LogSynthesis, Error, "Convolution num input channel mismatch. Expected %d, got %d", GetNumInputChannels(), InNumInputChannels);
			return;
		}
		else if (InNumOutputChannels != GetNumOutputChannels())
		{
			UE_LOGF(LogSynthesis, Error, "Convolution num output channel mismatch. Expected %d, got %d", GetNumOutputChannels(), InNumOutputChannels);
			return;
		}

		// Process all available audio that completes a block.
		InputBlockBuffer->AddSamples(InputAudio, NumInputSamples);

		while (InputBlockBuffer->GetNumAvailable() >= NumInputSamplesPerBlock)
		{
			const float* InputBlock = InputBlockBuffer->InspectSamples(NumInputSamplesPerBlock);

			if (nullptr != InputBlock)
			{
				ProcessAudioBlock(InputBlock, InNumInputChannels, InterleavedOutputBlock, InNumOutputChannels);
				OutputBlockBuffer->AddSamples(InterleavedOutputBlock.GetData(), InterleavedOutputBlock.Num());
			}

			InputBlockBuffer->RemoveSamples(NumInputSamplesPerBlock);
		}

		// Pop off all audio that's ready to go.
		int32 NumToPop = FMath::Min(OutputBlockBuffer->GetNumAvailable(), NumOutputSamples);

		if (NumToPop > 0)
		{
			int32 OutputCopyOffset = NumOutputSamples - NumToPop;
			const float* OutputPtr = OutputBlockBuffer->InspectSamples(NumToPop);
			if (nullptr != OutputPtr)
			{
				FMemory::Memcpy(&OutputAudio[OutputCopyOffset], OutputPtr, NumToPop * sizeof(float));
			}
			OutputBlockBuffer->RemoveSamples(NumToPop);
		}
	}

	void FConvolutionReverb::ProcessAudioBlock(const float* InputAudio, int32 InNumInputChannels, FAlignedFloatBuffer& OutputAudio, int32 InNumOutputChannels)
	{
		check(nullptr != InputAudio);
		check(ConvolutionAlgorithm.IsValid());

		const int32 NumFrames = ConvolutionAlgorithm->GetNumSamplesInBlock();
		const int32 NumInputSamples = InNumInputChannels * NumFrames;
		const int32 NumOutputSamples = InNumOutputChannels * NumFrames;

		TArrayView<const float> InputView(InputAudio, NumInputSamples);

		// De-interleave Input buffer into scratch input buffer
		DeinterleaveBuffer(InputDeinterleaveBuffers, InputView, InNumInputChannels);

		if (bIsConvertingInputChannelFormat)
		{
			const int32 NumConvertedChannels = InputChannelFormatConverter->GetOutputFormat().NumChannels;

			// Perform input mixing
			InputChannelFormatConverter->ProcessAudio(InputDeinterleaveBuffers, InputChannelConverterBuffers);

			// setup pointers for convolution in case any reallocation has happened.
			for (int32 i = 0; i < NumConvertedChannels; i++)
			{
				InputBufferPtrs[i] = InputChannelConverterBuffers[i].GetData();
			}
		}
		else
		{
			// setup pointers for convolution in case any reallocation has happened.
			for (int32 i = 0; i < InNumInputChannels; i++)
			{
				InputBufferPtrs[i] = InputDeinterleaveBuffers[i].GetData();
			}
		}


		// Setup buffers in case we're doing output mixing.
		if (bIsConvertingOutputChannelFormat)
		{
			const int32 NumUnconvertedChannels = OutputChannelFormatConverter.GetInputFormat().NumChannels;
			for (int32 i = 0; i < NumUnconvertedChannels; i++)
			{
				OutputBufferPtrs[i] = OutputChannelConverterBuffers[i].GetData();
			}
		}
		else
		{
			for (int32 i = 0; i < InNumOutputChannels; i++)
			{
				OutputBufferPtrs[i] = OutputDeinterleaveBuffers[i].GetData();
			}
		}

		// get next buffer of convolution output (store in scratch output buffer)
		ConvolutionAlgorithm->ProcessAudioBlock(InputBufferPtrs.GetData(), OutputBufferPtrs.GetData());


		if (bIsConvertingOutputChannelFormat)
		{
			OutputChannelFormatConverter.ProcessAudio(OutputChannelConverterBuffers, OutputDeinterleaveBuffers);
		}
			
		// re-interleave scratch output buffer into final output buffer
		InterleaveBuffer(OutputAudio, OutputDeinterleaveBuffers, InNumOutputChannels);

		// Apply final gain
		ArrayMultiplyByConstantInPlace(OutputAudio, OutputGain);
	}

	int32 FConvolutionReverb::GetNumInputChannels() const
	{
		if (!ConvolutionAlgorithm.IsValid())
		{
			return 0;
		}

		if (InputChannelFormatConverter.IsValid())
		{
			// Input audio is transformed to match convolution algorithm.
			return InputChannelFormatConverter->GetInputFormat().NumChannels;
		}
		else
		{
			// Input audio is fed directly into convolution algorithm.
			return ConvolutionAlgorithm->GetNumAudioInputs();
		}
	}

	int32 FConvolutionReverb::GetNumOutputChannels() const
	{
		if (!ConvolutionAlgorithm.IsValid())
		{
			return 0;
		}

		if (bIsConvertingOutputChannelFormat)
		{
			// Output audio is transformed from convolution output back to desired format.
			return OutputChannelFormatConverter.GetOutputFormat().NumChannels;
		}

		// Convolution algo output is output directly.
		return ConvolutionAlgorithm->GetNumAudioOutputs();
	}

	void FConvolutionReverb::Update(bool bFadeToParams)
	{
		OutputGain = Settings.NormalizationVolume;

		// Update output mixing parameters. 
		OutputChannelFormatConverter.SetRearChannelBleed(Settings.RearChannelBleed, bFadeToParams);
		OutputChannelFormatConverter.SetRearChannelFlip(Settings.bRearChannelFlip, bFadeToParams);
	}

	void FConvolutionReverb::ResizeProcessingBuffers()
	{
		int32 NumConvOutputChannels = 0;
		int32 NumOutputChannels = GetNumOutputChannels();

		int32 NumConvInputChannels = 0;
		int32 NumInputChannels = GetNumInputChannels();

		int32 NumFrames = 0;

		NumInputSamplesPerBlock = 0;
		NumOutputSamplesPerBlock = 0;

		if (ConvolutionAlgorithm.IsValid())
		{
			NumConvOutputChannels = ConvolutionAlgorithm->GetNumAudioOutputs();
			NumConvInputChannels = ConvolutionAlgorithm->GetNumAudioInputs();
			NumFrames = ConvolutionAlgorithm->GetNumSamplesInBlock();

			NumInputSamplesPerBlock = NumInputChannels * NumFrames;
			NumOutputSamplesPerBlock = NumOutputChannels * NumFrames;

			// Buffers used to deinterleave input audio.
			ResizeArrayOfBuffers(InputDeinterleaveBuffers, NumInputChannels, NumFrames);

			// Buffers used to interleave output audio
			ResizeArrayOfBuffers(OutputDeinterleaveBuffers, NumOutputChannels, NumFrames);

			// Buffers used to mix input audio to conv algo
			if (InputChannelFormatConverter.IsValid())
			{
				ResizeArrayOfBuffers(InputChannelConverterBuffers, NumConvInputChannels, NumFrames);
			}

			// Buffers used to mix conv output to output audio.
			if (bIsConvertingOutputChannelFormat)
			{
				ResizeArrayOfBuffers(OutputChannelConverterBuffers, NumConvOutputChannels, NumFrames);
			}

			// Buffer pointers fed into convolution algorithm
			InputBufferPtrs.SetNumZeroed(NumConvInputChannels);
			OutputBufferPtrs.SetNumZeroed(NumConvOutputChannels);
		}
	}

	void FConvolutionReverb::ResizeArrayOfBuffers(TArray<FAlignedFloatBuffer>& InArrayOfBuffers, int32 MinNumBuffers, int32 NumFrames) const
	{
		while (InArrayOfBuffers.Num() < MinNumBuffers)
		{
			InArrayOfBuffers.Emplace();
		}

		for (int32 i = 0; i < MinNumBuffers; i++)
		{
			FAlignedFloatBuffer& Buffer = InArrayOfBuffers[i];
			Buffer.Reset();
			if (NumFrames > 0)
			{
				Buffer.AddUninitialized(NumFrames);
			}
		}
	}

	void FConvolutionReverb::ResizeBlockBuffers()
	{
		// Block buffers handle buffering of calls to ProcessAudio().  Internally, 
		// processing is done strictly on frame boundaries matching the ConvolutionAlgorithm
		// in the ProcessAudioBlock() call. These block buffers handle storage and logic 
		// to ensure that buffers chunked appropriately in ProcessAudio() and then
		// fed to ProcessAudioBlock() correctly. 

		int32 AlgoBlockSize = 0;
		int32 NumInputChannels = GetNumInputChannels();
		int32 NumOutputChannels = GetNumOutputChannels();

		if (ConvolutionAlgorithm.IsValid())
		{
			AlgoBlockSize = ConvolutionAlgorithm->GetNumSamplesInBlock();
		}

		// ChunkSize is a guess at the number of frames that will be processed
		// during a call to ProcessAudio and ProcessAudioBlock. It's either the 
		// larger of the expected callback size or the algorithm block size.
		int32 ChunkSize = FMath::Max(ExpectedNumFramesPerCallback, AlgoBlockSize);

		// Enforce a lower bound for chunk size.
		ChunkSize = FMath::Max(128, ChunkSize);

		// Create block buffers based upon input/output number of samples with
		// added capacity for buffering and a lower bound. 

		int32 InputCapacity = FMath::Max(8192, 2 * ChunkSize * NumInputChannels);
		int32 InputMaxInspect = FMath::Max(1024, AlgoBlockSize * NumInputChannels);

		InputBlockBuffer = MakeUnique<FAlignedBlockBuffer>(InputCapacity, InputMaxInspect);

		int32 OutputCapacity = FMath::Max(8192, 2 * ChunkSize * NumOutputChannels);
		int32 OutputMaxInspect = FMath::Max(1024, ExpectedNumFramesPerCallback * NumOutputChannels);

		OutputBlockBuffer = MakeUnique<FAlignedBlockBuffer>(OutputCapacity, OutputMaxInspect);
	}

	void FConvolutionReverb::InterleaveBuffer(FAlignedFloatBuffer& OutBuffer, const TArray<FAlignedFloatBuffer>& InputBuffers, const int32 NumChannels)
	{
		check(InputBuffers.Num() >= NumChannels);

		OutBuffer.Reset();

		if (InputBuffers.Num() < 1)
		{
			return;
		}

		int32 NumFrames = InputBuffers[0].Num();
		int32 NumOutputSamples = NumFrames * NumChannels;

		if (NumOutputSamples > 0)
		{
			OutBuffer.AddUninitialized(NumOutputSamples);
		}

		float* OutData = OutBuffer.GetData();
		for (int32 i = 0; i < NumChannels; ++i)
		{
			check(InputBuffers[i].Num() == NumFrames);

			const float* InData = InputBuffers[i].GetData();

			int32 OutPos = i;

			for (int32 j = 0; j < NumFrames; j++)
			{
				OutData[OutPos] = InData[j];
				OutPos += NumChannels;
			}
		}
	}

	void FConvolutionReverb::DeinterleaveBuffer(TArray<FAlignedFloatBuffer>& OutputBuffers, TArrayView<const float> InputBuffer, const int32 NumChannels)
	{
		check(OutputBuffers.Num() >= NumChannels);

		int32 NumFrames = InputBuffer.Num();

		if (NumChannels > 0)
		{
			NumFrames /= NumChannels;
		}

		const float* InputData = InputBuffer.GetData();

		for (int32 i = 0; i < NumChannels; i++)
		{
			FAlignedFloatBuffer& OutBuffer = OutputBuffers[i];

			check(OutBuffer.Num() == NumFrames);

			float* OutData = OutBuffer.GetData();

			int32 InputPos = i;

			for (int32 j = 0; j < NumFrames; j++)
			{
				OutData[j] = InputData[InputPos];
				InputPos += NumChannels;
			}
		}
	}
}
