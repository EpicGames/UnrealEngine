// Copyright Epic Games, Inc. All Rights Reserved.

#include "EffectConvolutionReverb.h"
#include "Async/AsyncWork.h"
#include "DSP/AlignedBlockBuffer.h" // IWYU pragma: keep
#include "SynthesisModule.h"
#include "Misc/Parse.h"
#include "Misc/CommandLine.h"
#include "HAL/IConsoleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EffectConvolutionReverb)


static int32 bDisabledHardwareConvolution = 0;
FAutoConsoleVariableRef CVarDisableHardwareConvolution(
	TEXT("au.DisableHardwareConvolution"),
	bDisabledHardwareConvolution,
	TEXT("Disables hardware convolution. Will not change already instantiated hardware accelerated convolution.\n")
	TEXT("1: New instances of hardware-acclerated convolution is disabled. 0: Hardware convolution is not disabled."),
	ECVF_Default);


// Command line arg to disable hardware convolution

static bool IsHardwareConvolutionDisabled()
{	
	static bool bDisableConvolutionHardareAcceleration = FParse::Param(FCommandLine::Get(), TEXT("DisableHardwareConvolution"));
	return bDisableConvolutionHardareAcceleration || bDisabledHardwareConvolution;
}


UAudioImpulseResponse::UAudioImpulseResponse()
	: NumChannels(0)
	, SampleRate(0)
	, NormalizationVolumeDb(-24.f)
{
}

#if WITH_EDITORONLY_DATA
void UAudioImpulseResponse::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	OnObjectPropertyChanged.Broadcast(PropertyChangedEvent);
}
#endif

void UAudioImpulseResponse::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

#if WITH_EDITORONLY_DATA
	if (Ar.IsLoading())
	{
		bIsEvenChannelCount = NumChannels % 2 == 0;
	}
#endif // WITH_EDITORONLY_DATA
}

namespace AudioConvReverbIntrinsics
{
	// Task for creating convolution algorithm object.
	class SYNTHESIS_API FCreateConvolutionReverbTask : public FNonAbandonableTask
	{
		// This task can delete itself
		friend class FAutoDeleteAsyncTask<FCreateConvolutionReverbTask>;

	public:
		FCreateConvolutionReverbTask(
			TWeakPtr<Audio::FEffectConvolutionReverb, ESPMode::ThreadSafe> InEffectPtr,
			Audio::FConvolutionReverbInitData&& InInitData,
			Audio::FConvolutionReverbSettings& InSettings,
			const FVersionData& InVersionData)
			: EffectPtr(InEffectPtr)
			, InitData(MoveTemp(InInitData))
			, Settings(InSettings)
			, VersionData(InVersionData)
		{
		}

		void DoWork()
		{
			using namespace Audio;

			// Build the convolution reverb object. 
			TUniquePtr<FConvolutionReverb> ConvReverb = FConvolutionReverb::CreateConvolutionReverb(InitData, Settings);

			TSharedPtr<FEffectConvolutionReverb, ESPMode::ThreadSafe> EffectSharedPtr = EffectPtr.Pin();
			if (EffectSharedPtr.IsValid())
			{
				// Check that the effect still exists.
				EffectSharedPtr->EnqueueNewReverb(MoveTemp(ConvReverb), VersionData);
			}
		}

		FORCEINLINE TStatId GetStatId() const { RETURN_QUICK_DECLARE_CYCLE_STAT(CreateConvolutionReverbTask, STATGROUP_ThreadPoolAsyncTasks); }

	private:
		TWeakPtr<FEffectConvolutionReverb, ESPMode::ThreadSafe> EffectPtr;

		FConvolutionReverbInitData InitData;
		FConvolutionReverbSettings Settings;
		FVersionData VersionData;
	};

	typedef FAutoDeleteAsyncTask<FCreateConvolutionReverbTask> FCreateReverbTask;
}

namespace Audio
{
	FConvolutionReverbInitData FEffectConvolutionReverb::CreateConvolutionReverbInitData()
	{
		FScopeLock ConvReverbInitDataLock(&ConvReverbInitDataCriticalSection);

		const int32 NumInputChannelsLocal = NumInputChannels.Load();
		const int32 NumOutputChannelsLocal = NumOutputChannels.Load();

		ConvReverbInitData.InputAudioFormat.NumChannels = NumInputChannelsLocal;
		ConvReverbInitData.OutputAudioFormat.NumChannels = NumOutputChannelsLocal;

		// Mono to true stereo is a special case where we have the true stereo flag
		// checked but only have 2 IRs
		bool bIsMonoToTrueStereo = ConvReverbInitData.bIsImpulseTrueStereo && (ConvReverbInitData.AlgorithmSettings.NumImpulseResponses == 2);

		// Determine correct input channel counts 
		if (ConvReverbInitData.bMixInputChannelFormatToImpulseResponseFormat)
		{
			if (bIsMonoToTrueStereo)
			{
				// Force mono input.
				ConvReverbInitData.AlgorithmSettings.NumInputChannels = 1;
			}
			else if (ConvReverbInitData.bIsImpulseTrueStereo)
			{
				// If performing true stereo, force input to be stereo
				ConvReverbInitData.AlgorithmSettings.NumInputChannels = 2;
			}
			else
			{
				ConvReverbInitData.AlgorithmSettings.NumInputChannels = ConvReverbInitData.AlgorithmSettings.NumImpulseResponses;
			}
		}
		else
		{
			ConvReverbInitData.AlgorithmSettings.NumInputChannels = NumInputChannelsLocal;
		}


		// Determine correct output channel count
		if (bIsMonoToTrueStereo)
		{
			ConvReverbInitData.AlgorithmSettings.NumOutputChannels = 2;
		}
		else if (ConvReverbInitData.bIsImpulseTrueStereo)
		{
			// If IRs are true stereo, divide output channel count in half.
			ConvReverbInitData.AlgorithmSettings.NumOutputChannels = ConvReverbInitData.AlgorithmSettings.NumImpulseResponses / 2;
		}
		else
		{
			ConvReverbInitData.AlgorithmSettings.NumOutputChannels = ConvReverbInitData.AlgorithmSettings.NumImpulseResponses;
		}

		// Determine longest IR
		ConvReverbInitData.AlgorithmSettings.MaxNumImpulseResponseSamples = 0;

		if (ConvReverbInitData.AlgorithmSettings.NumImpulseResponses > 0)
		{
			// Determine sample rate ratio in order to calculate the final IR num samples. 
			float SampleRateRatio = 1.f;
			if (ConvReverbInitData.ImpulseSampleRate > 0.f)
			{
				SampleRateRatio = ConvReverbInitData.TargetSampleRate / ConvReverbInitData.ImpulseSampleRate;
			}

			ConvReverbInitData.AlgorithmSettings.MaxNumImpulseResponseSamples = FMath::CeilToInt(SampleRateRatio * ConvReverbInitData.Samples.Num() / ConvReverbInitData.AlgorithmSettings.NumImpulseResponses) + 256;
		}

		// Setup gain matrix.
		ConvReverbInitData.GainMatrix.Reset();

		if (NumInputChannelsLocal == 1)
		{
			ConvReverbInitData.GainMatrix.Emplace(0, 0, 0, 1.f);
		}
		else if (bIsMonoToTrueStereo)
		{
			ConvReverbInitData.GainMatrix.Emplace(0, 0, 0, 1.f);
			ConvReverbInitData.GainMatrix.Emplace(0, 1, 1, 1.f);
		}
		else if (ConvReverbInitData.bIsImpulseTrueStereo)
		{
			// True stereo treats first group of IRs 
			int32 NumPairs = ConvReverbInitData.AlgorithmSettings.NumImpulseResponses / 2;

			for (int32 PairIndex = 0; PairIndex < NumPairs; PairIndex++)
			{
				int32 LeftImpulseIndex = PairIndex;
				int32 RightImpulseIndex = NumPairs + PairIndex;

				ConvReverbInitData.GainMatrix.Emplace(0, LeftImpulseIndex, PairIndex, 1.f);
				ConvReverbInitData.GainMatrix.Emplace(1, RightImpulseIndex, PairIndex, 1.f);
			}
		}
		else
		{
			// Set up a 1-to-1 mapping. 
			int32 MinChannelCount = FMath::Min3(ConvReverbInitData.AlgorithmSettings.NumInputChannels, ConvReverbInitData.AlgorithmSettings.NumOutputChannels, ConvReverbInitData.AlgorithmSettings.NumImpulseResponses);

			for (int32 i = 0; i < MinChannelCount; i++)
			{
				ConvReverbInitData.GainMatrix.Emplace(i, i, i, 1.f);
			}
		}

		if (IsHardwareConvolutionDisabled())
		{
			if (ConvReverbInitData.AlgorithmSettings.bEnableHardwareAcceleration)
			{
				UE_LOGF(LogSynthesis, Display, "Disabling hardware acceleration due to cvar or command line argument.");
				ConvReverbInitData.AlgorithmSettings.bEnableHardwareAcceleration = false;
			}
		}

		return ConvReverbInitData;
	}

	void FEffectConvolutionReverb::BuildReverb()
	{
		using namespace AudioConvReverbIntrinsics;
		FVersionData Version = UpdateVersion();

		FConvolutionReverbSettings Settings;
		Params.CopyParams(Settings);

		FConvolutionReverbInitData InitData = CreateConvolutionReverbInitData();

		(new FCreateReverbTask(AsWeak(), MoveTemp(InitData), Settings, VersionData))->StartBackgroundTask();
	}

	void FEffectConvolutionReverb::Init()
	{
		FConvolutionReverbSettings Settings;
		Params.CopyParams(Settings);

		// Create the convolution algorithm init data
		FConvolutionReverbInitData ConvolutionInitData = CreateConvolutionReverbInitData();

		SetReverb(FConvolutionReverb::CreateConvolutionReverb(ConvolutionInitData, Settings));
	}

	void FEffectConvolutionReverb::SetSettings(const FConvolutionReverbSettings& InSettings)
	{
		Params.SetParams(InSettings);

		if (Reverb.IsValid())
		{
			Reverb->SetSettings(InSettings);
		}
	}

	void FEffectConvolutionReverb::UpdateParameters()
	{
		Audio::FConvolutionReverbSettings NewSettings;
		if (Reverb.IsValid() && Params.GetParams(&NewSettings))
		{
			Reverb->SetSettings(NewSettings);
		}
	}

	void FEffectConvolutionReverb::SetSampleRate(const float InSampleRate)
	{
		FScopeLock Lock(&ConvReverbInitDataCriticalSection);

		SampleRate = InSampleRate;
		ConvReverbInitData.TargetSampleRate = SampleRate;
	}

	void FEffectConvolutionReverb::SetInitData(const FConvolutionReverbInitData& InData)
	{
		FScopeLock Lock(&ConvReverbInitDataCriticalSection);

		ConvReverbInitData = InData;
		ConvReverbInitData.TargetSampleRate = SampleRate;
	}

	void FEffectConvolutionReverb::SetReverb(TUniquePtr<Audio::FConvolutionReverb> InReverb)
	{
		Reverb = MoveTemp(InReverb);
	}

	void FEffectConvolutionReverb::SetBypass(const bool InBypass)
	{
		bBypass = InBypass;
	}

	void FEffectConvolutionReverb::UpdateChannelCount(const int32 InNumInputChannels, const int32 InNumOutputChannels)
	{
		int32 ExpectedNumInputChannels = 0;
		int32 ExpectedNumOutputChannels = 0;

		if (Reverb.IsValid())
		{
			ExpectedNumInputChannels = Reverb->GetNumInputChannels();
			ExpectedNumOutputChannels = Reverb->GetNumOutputChannels();
		}

		// Check if there is a channel mismatch between the convolution reverb and the input/output data
		const bool bNumChannelsMismatch = !IsChannelCountUpToDate(InNumInputChannels, InNumOutputChannels);
		const bool bRequestedChannelCountIsValid = InNumInputChannels > 0 && InNumOutputChannels > 0;

		if (bNumChannelsMismatch && bRequestedChannelCountIsValid)
		{
			int32 NumInputChannelsTemp = NumInputChannels.Load();
			int32 NumOutputChannelsTemp = NumOutputChannels.Load();

			// check the NumInputChannelsTemp/NumOutputChannelsTemp to check if we've
			// already tried and failed to create an algorithm for this channel configuration.
			bool bShouldCreateNewAlgo = (NumInputChannelsTemp != InNumInputChannels) || (NumOutputChannelsTemp != InNumOutputChannels);

			if (bShouldCreateNewAlgo)
			{
				UE_LOGF(LogSynthesis, Log, "Creating new convolution algorithm due to channel count update. Num Inputs %d -> %d. Num Outputs %d -> %d"
					, ExpectedNumInputChannels, InNumInputChannels, ExpectedNumOutputChannels, InNumOutputChannels);

				// These should only be updated here and in the constructor since we use them
				// to see if we've tried this channel configuration already.
				NumInputChannels.Store(InNumInputChannels);
				NumOutputChannels.Store(InNumOutputChannels);

				// Create the convolution reverb in an async task.
				BuildReverb();
			}
		}
	}

	void FEffectConvolutionReverb::EnqueueNewReverb(TUniquePtr<FConvolutionReverb> InReverb, const AudioConvReverbIntrinsics::FVersionData& InVersionData)
	{
		// This queued reverb will get hotswapped the next time ProcessAudio comes around on the render thread
		FScopeLock Lock(&VersionDataCriticalSection);

		QueuedVersionData = InVersionData;
		QueuedReverb = MoveTemp(InReverb);
	}

	void FEffectConvolutionReverb::DequeueNewReverb()
	{
		FScopeLock Lock(&VersionDataCriticalSection);
		
		if (QueuedReverb.IsValid() && VersionData == QueuedVersionData)
		{
			Reverb = MoveTemp(QueuedReverb);
		}
	}

	bool FEffectConvolutionReverb::IsVersionCurrent(const AudioConvReverbIntrinsics::FVersionData& InVersionData) const
	{
		FScopeLock VersionDataLock(&VersionDataCriticalSection);

		return InVersionData == VersionData;
	}

	AudioConvReverbIntrinsics::FVersionData FEffectConvolutionReverb::UpdateVersion()
	{
		AudioConvReverbIntrinsics::FVersionData VersionDataCopy;
		FScopeLock VersionDataLock(&VersionDataCriticalSection);

		VersionData.ConvolutionID++;
		VersionDataCopy = VersionData;

		return VersionDataCopy;
	}

	bool FEffectConvolutionReverb::IsChannelCountUpToDate(const int32 InNumInputChannels, const int32 InNumOutputChannels) const
	{
		if (Reverb.IsValid() == false)
		{
			return false;
		}

		const int32 ExpectedNumInputChannels = Reverb->GetNumInputChannels();
		const int32 ExpectedNumOutputChannels = Reverb->GetNumOutputChannels();

		return ExpectedNumInputChannels == InNumInputChannels && ExpectedNumOutputChannels == InNumOutputChannels;
	}

	void FEffectConvolutionReverb::ProcessAudio(int32 InNumInputChannels, const float* InputAudio, int32 InNumOutputChannels, float* OutputAudio, const int32 InNumFrames)
	{
		DequeueNewReverb();

		const bool bShouldProcessConvReverb = !bBypass && Reverb.IsValid() && IsChannelCountUpToDate(InNumInputChannels, InNumOutputChannels);

		if (bShouldProcessConvReverb)
		{
			UpdateParameters();
			Reverb->ProcessAudio(InNumInputChannels, InputAudio, InNumOutputChannels, OutputAudio, InNumFrames);
		}
		else if (bBypass)
		{
			FMemory::Memcpy(OutputAudio, InputAudio, InNumFrames * InNumOutputChannels * sizeof(float));
		}
		else
		{
			// Zero output data. Do *not* trigger rebuild here in case one is already in flight or simply because one cannot be built.
			if (InNumFrames > 0)
			{
				FMemory::Memzero(OutputAudio, InNumFrames * InNumOutputChannels * sizeof(float));
			}
		}
	}
}
