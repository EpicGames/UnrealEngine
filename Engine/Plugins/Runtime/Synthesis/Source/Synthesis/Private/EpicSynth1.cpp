// Copyright Epic Games, Inc. All Rights Reserved.

#include "EpicSynth1.h"
#include "SynthesisModule.h"

namespace Audio
{
#define SYNTH_DEBUG_MODE 0

	FEpicSynth1Voice::FEpicSynth1Voice()
		: CurrentFilter(nullptr)
		, MidiNote(INDEX_NONE)
		, VoiceId(INDEX_NONE)
		, ControlSampleCount(0)
		, DurationSampleCount(-1)
		, CurrentSampleCount(0)
		, VoiceGeneration(INDEX_NONE)
		, ParentSynth(nullptr)
		, bIsFinished(true)
		, bIsActive(false)
	{
		CurrentPatchType[0] = ESynthLFOPatchType::PatchToNone;
		CurrentPatchType[1] = ESynthLFOPatchType::PatchToNone;

		CurrentModPatchType = ESynthModEnvPatch::PatchToNone;
		CurrentModBiasPatchType = ESynthModEnvBiasPatch::PatchToNone;
	}

	FEpicSynth1Voice::~FEpicSynth1Voice()
	{

	}

	void FEpicSynth1Voice::Init(FEpicSynth1* InParentSynth, const int32 InVoiceId)
	{
		ParentSynth = InParentSynth;
		VoiceId = InVoiceId;
		ControlSampleCount = 0;

		PortamentoFrequency.Init(ParentSynth->SampleRate);

		const float SampleRate = ParentSynth->SampleRate;
		const float ControlSampleRate = ParentSynth->ControlSampleRate;
		const EFilter::Type FilterType = ParentSynth->FilterType;
		const float BaseFilterFreq = ParentSynth->BaseFilterFreq;

		FModulationMatrix* ModMatrix = &ParentSynth->ModMatrix;

		for (int32 i = 0; i < NumLFOs; ++i)
		{
			LFO[i].Init(ControlSampleRate, VoiceId, ModMatrix);
			LFO[i].SetBipolar(false);
		}
		GainEnv.Init(ControlSampleRate, VoiceId, ModMatrix);
		ModEnv.Init(ControlSampleRate, VoiceId, ModMatrix);

		for (int32 i = 0; i < NumOscillators; ++i)
		{
			Oscil[i].Init(SampleRate, VoiceId, ModMatrix);
			OscilPan[i].Init(VoiceId, ModMatrix);
		}

		// Setup Osc1 to be a leader to Osc2 
		Oscil[0].SetFollowerOsc(&Oscil[1]);

		Amp.Init(VoiceId, ModMatrix);

		OnePoleFilter.Init(SampleRate, 2, VoiceId, ModMatrix);
		OnePoleFilter.SetFilterType(FilterType);
		OnePoleFilter.SetFrequency(BaseFilterFreq);

		StateVarFilter.Init(SampleRate, 2, VoiceId, ModMatrix);
		StateVarFilter.SetFilterType(FilterType);
		StateVarFilter.SetFrequency(BaseFilterFreq);

		LadderFilter.Init(SampleRate, 2, VoiceId, ModMatrix);
		LadderFilter.SetFilterType(FilterType);
		LadderFilter.SetFrequency(BaseFilterFreq);

		CurrentFilter = &OnePoleFilter;

		// Setup the mod matrix

		FPatchDestination OscilFreqDest[NumOscillators];
		FPatchDestination OscilPWDest[NumOscillators];

		for (int32 i = 0; i < NumOscillators; ++i)
		{
			OscilFreqDest[i] = Oscil[i].GetModDestFrequency();
			OscilPWDest[i] = Oscil[i].GetModDestPulseWidth();
		}

		// Only 1 filter and amp DSP object
		FPatchDestination Filter_OnePole_Freq = OnePoleFilter.GetModDestCutoffFrequency();
		FPatchDestination Filter_StateVar_Freq = StateVarFilter.GetModDestCutoffFrequency();
		FPatchDestination Filter_Ladder_Freq = LadderFilter.GetModDestCutoffFrequency();
		FPatchDestination Filter_StateVar_Q = StateVarFilter.GetModDestQ();
		FPatchDestination Filter_Ladder_Q = LadderFilter.GetModDestQ();
		FPatchDestination Amp_GainEnv = Amp.GetModDestGainEnv();
		FPatchDestination Amp_Gain = Amp.GetModDestGainScale();
		FPatchDestination Amp_Pan = Amp.GetModDestPan();

		FPatchDestination LFO1Freq = LFO[0].GetModDestFrequency();
		FPatchDestination LFO2Freq = LFO[1].GetModDestFrequency();
		FPatchDestination LFO1Gain = LFO[0].GetModDestGain();
		FPatchDestination LFO2Gain = LFO[1].GetModDestGain();

		FPatch* Patch = nullptr;

		FPatchSource ModEnv_ModSourceEnv = ModEnv.GetModSourceEnv();
		FPatchSource ModEnv_ModSourceBiasEnv = ModEnv.GetModSourceBiasEnv();

		// Setup the ModEnv patches
		Patch = &ModEnvPatches[(int32)ESynthModEnvPatch::PatchToOscFreq];
		Patch->bEnabled = false;
		Patch->Source = ModEnv_ModSourceEnv;
		Patch->Destinations.Add(OscilFreqDest[0]);
		Patch->Destinations.Add(OscilFreqDest[1]);
		Patch->SetName("ESynthModEnvPatch::PatchToOscFreq");
		ModMatrix->AddPatch(VoiceId, Patch);

		Patch = &ModEnvPatches[(int32)ESynthModEnvPatch::PatchToFilterFreq];
		Patch->bEnabled = false;
		Patch->Source = ModEnv_ModSourceEnv;
		Patch->Destinations.Add(Filter_OnePole_Freq);
		Patch->Destinations.Add(Filter_StateVar_Freq);
		Patch->Destinations.Add(Filter_Ladder_Freq);
		Patch->SetName("ESynthModEnvPatch::PatchToFilterFreq");
		ModMatrix->AddPatch(VoiceId, Patch);

		Patch = &ModEnvPatches[(int32)ESynthModEnvPatch::PatchToFilterQ];
		Patch->bEnabled = false;
		Patch->Source = ModEnv_ModSourceEnv;
		Patch->Destinations.Add(Filter_StateVar_Q);
		Patch->Destinations.Add(Filter_Ladder_Q);
		Patch->SetName("ESynthModEnvPatch::PatchToFilterQ");
		ModMatrix->AddPatch(VoiceId, Patch);

		Patch = &ModEnvPatches[(int32)ESynthModEnvPatch::PatchToLFO1Gain];
		Patch->bEnabled = false;
		Patch->Source = ModEnv_ModSourceEnv;
		Patch->Destinations.Add(LFO1Gain);
		Patch->SetName("ESynthModEnvPatch::PatchToLFO1Gain");
		ModMatrix->AddPatch(VoiceId, Patch);

		Patch = &ModEnvPatches[(int32)ESynthModEnvPatch::PatchToLFO2Gain];
		Patch->bEnabled = false;
		Patch->Source = ModEnv_ModSourceEnv;
		Patch->Destinations.Add(LFO2Gain);
		Patch->SetName("ESynthModEnvPatch::PatchToLFO2Gain");
		ModMatrix->AddPatch(VoiceId, Patch);

		Patch = &ModEnvPatches[(int32)ESynthModEnvPatch::PatchToLFO1Freq];
		Patch->bEnabled = false;
		Patch->Source = ModEnv_ModSourceEnv;
		Patch->Destinations.Add(LFO1Freq);
		Patch->SetName("ESynthModEnvPatch::PatchToLFO1Freq");
		ModMatrix->AddPatch(VoiceId, Patch);

		Patch = &ModEnvPatches[(int32)ESynthModEnvPatch::PatchToLFO2Freq];
		Patch->bEnabled = false;
		Patch->Source = ModEnv_ModSourceEnv;
		Patch->Destinations.Add(LFO1Freq);
		Patch->SetName("ESynthModEnvPatch::PatchToLFO2Freq");
		ModMatrix->AddPatch(VoiceId, Patch);

		Patch = &ModEnvBiasPatches[(int32)ESynthModEnvBiasPatch::PatchToOscFreq];
		Patch->bEnabled = false;
		Patch->Source = ModEnv_ModSourceBiasEnv;
		Patch->Destinations.Add(OscilFreqDest[0]);
		Patch->Destinations.Add(OscilFreqDest[1]);
		Patch->SetName("ESynthModEnvBiasPatch::PatchToOscFreq");
		ModMatrix->AddPatch(VoiceId, Patch);

		Patch = &ModEnvBiasPatches[(int32)ESynthModEnvBiasPatch::PatchToFilterFreq];
		Patch->bEnabled = false;
		Patch->Source = ModEnv_ModSourceBiasEnv;
		Patch->Destinations.Add(Filter_OnePole_Freq);
		Patch->Destinations.Add(Filter_StateVar_Freq);
		Patch->Destinations.Add(Filter_Ladder_Freq);
		Patch->SetName("ESynthModEnvBiasPatch::PatchToFilterFreq");
		ModMatrix->AddPatch(VoiceId, Patch);

		Patch = &ModEnvBiasPatches[(int32)ESynthModEnvBiasPatch::PatchToFilterQ];
		Patch->bEnabled = false;
		Patch->Source = ModEnv_ModSourceBiasEnv;
		Patch->Destinations.Add(Filter_StateVar_Q);
		Patch->Destinations.Add(Filter_Ladder_Q);
		Patch->SetName("ESynthModEnvBiasPatch::PatchToFilterQ");
		ModMatrix->AddPatch(VoiceId, Patch);

		Patch = &ModEnvBiasPatches[(int32)ESynthModEnvBiasPatch::PatchToLFO1Gain];
		Patch->bEnabled = false;
		Patch->Source = ModEnv_ModSourceBiasEnv;
		Patch->Destinations.Add(LFO1Gain);
		Patch->SetName("ESynthModEnvBiasPatch::PatchToLFO1Gain");
		ModMatrix->AddPatch(VoiceId, Patch);

		Patch = &ModEnvBiasPatches[(int32)ESynthModEnvBiasPatch::PatchToLFO2Gain];
		Patch->bEnabled = false;
		Patch->Source = ModEnv_ModSourceBiasEnv;
		Patch->Destinations.Add(LFO2Gain);
		Patch->SetName("ESynthModEnvBiasPatch::PatchToLFO2Gain");
		ModMatrix->AddPatch(VoiceId, Patch);

		Patch = &ModEnvBiasPatches[(int32)ESynthModEnvBiasPatch::PatchToLFO1Freq];
		Patch->bEnabled = false;
		Patch->Source = ModEnv_ModSourceBiasEnv;
		Patch->Destinations.Add(LFO1Freq);
		Patch->SetName("ESynthModEnvBiasPatch::PatchToLFO1Freq");
		ModMatrix->AddPatch(VoiceId, Patch);

		Patch = &ModEnvBiasPatches[(int32)ESynthModEnvBiasPatch::PatchToLFO2Freq];
		Patch->bEnabled = false;
		Patch->Source = ModEnv_ModSourceBiasEnv;
		Patch->Destinations.Add(LFO1Freq);
		Patch->SetName("ESynthModEnvBiasPatch::PatchToLFO2Freq");
		ModMatrix->AddPatch(VoiceId, Patch);

		// Setup LFO patches
		for (int32 i = 0; i < NumLFOs; ++i)
		{
			FPatchSource NormalPhasePatchSource = LFO[i].GetModSourceNormalPhase();
			
			Patch = &LFOPatches[i][(int32)ESynthLFOPatchType::PatchToGain];
			Patch->bEnabled = false;
			Patch->Source = NormalPhasePatchSource;
			Patch->Destinations.Add(Amp_Gain);
			Patch->SetName("ELFOPatch::PatchToGain");
			// Note MinValue can be greater than MaxValue since this is really just a range value (i.e. this inverts the sign)
			ModMatrix->AddPatch(VoiceId, Patch);

			Patch = &LFOPatches[i][(int32)ESynthLFOPatchType::PatchToOscFreq];
			Patch->bEnabled = false;
			Patch->Source = NormalPhasePatchSource;
			Patch->SetName("ELFOPatch::PatchToOscFreq");

			for (int32 OscIndex = 0; OscIndex < NumOscillators; ++OscIndex)
			{
				Patch->Destinations.Add(OscilFreqDest[OscIndex]);
			}
			ModMatrix->AddPatch(VoiceId, Patch);

			Patch = &LFOPatches[i][(int32)ESynthLFOPatchType::PatchToOscPulseWidth];
			Patch->bEnabled = false;
			Patch->Source = NormalPhasePatchSource;
			Patch->SetName("ELFOPatch::PatchToOscPulseWidth");

			for (int32 OscIndex = 0; OscIndex < NumOscillators; ++OscIndex)
			{
				Patch->Destinations.Add(OscilPWDest[OscIndex]);
			}
			ModMatrix->AddPatch(VoiceId, Patch);

			Patch = &LFOPatches[i][(int32)ESynthLFOPatchType::PatchToFilterFreq];
			Patch->bEnabled = false;
			Patch->Source = NormalPhasePatchSource;
			Patch->Destinations.Add(Filter_OnePole_Freq);
			Patch->Destinations.Add(Filter_StateVar_Freq);
			Patch->Destinations.Add(Filter_Ladder_Freq);		

			Patch->SetName("ELFOPatch::PatchToFilterFreq");
			ModMatrix->AddPatch(VoiceId, Patch);

			Patch = &LFOPatches[i][(int32)ESynthLFOPatchType::PatchToFilterQ];
			Patch->bEnabled = false;
			Patch->Source = NormalPhasePatchSource;
			Patch->Destinations.Add(Filter_StateVar_Q);
			Patch->Destinations.Add(Filter_Ladder_Q);

			Patch->SetName("ELFOPatch::PatchToFilterQ");
			ModMatrix->AddPatch(VoiceId, Patch);


			Patch = &LFOPatches[i][(int32)ESynthLFOPatchType::PatchToOscPan];
			Patch->bEnabled = false;
			Patch->Source = NormalPhasePatchSource;
			Patch->Destinations.Add(Amp_Pan);
			Patch->SetName("ELFOPatch::PatchToOscPan");
			ModMatrix->AddPatch(VoiceId, Patch);
		}

		// Create patch from LFO1 to LFO2 frequency
		Patch = &LFOPatches[0][(int32)ESynthLFOPatchType::PatchLFO1ToLFO2Frequency];
		Patch->bEnabled = false;
		Patch->Source = LFO[0].GetModSourceNormalPhase();
		Patch->Destinations.Add(LFO2Freq);
		Patch->SetName("ELFOPatch::PatchLFO1ToLFO2Frequency");
		ModMatrix->AddPatch(VoiceId, Patch);

		// Create patch from LFO1 to LFO2 Gain
		Patch = &LFOPatches[0][(int32)ESynthLFOPatchType::PatchLFO1ToLFO2Gain];
		Patch->bEnabled = false;
		Patch->Source = LFO[0].GetModSourceNormalPhase();
		Patch->Destinations.Add(LFO2Gain);
		Patch->SetName("ELFOPatch::PatchLFO1ToLFO2Frequency");
		ModMatrix->AddPatch(VoiceId, Patch);

		// Setup envelope gain patch
		Env_To_Amp.bEnabled = true;
		Env_To_Amp.Source = GainEnv.GetModSourceEnv();
		Env_To_Amp.Destinations.Add(Amp_GainEnv);
		Env_To_Amp.SetName("PatchEnvToAmp");
		ModMatrix->AddPatch(VoiceId, &Env_To_Amp);
	}

	FPatchSource FEpicSynth1Voice::GetPatchSource(ESynth1PatchSource PatchSource)
	{
		switch (PatchSource)
		{
			case ESynth1PatchSource::LFO1:
				return LFO[0].GetModSourceNormalPhase();

			case ESynth1PatchSource::LFO2:
				return LFO[1].GetModSourceNormalPhase();

			case ESynth1PatchSource::Envelope:
				return ModEnv.GetModSourceEnv();

			case ESynth1PatchSource::BiasEnvelope:
				return ModEnv.GetModSourceBiasEnv();

			default:
				return FPatchSource();
		}
	}

	void FEpicSynth1Voice::GetPatchDestinations(ESynth1PatchDestination PatchDestination, TArray<FPatchDestination>& Destinations)
	{
		switch (PatchDestination)
		{
			case ESynth1PatchDestination::Osc1Gain: 
				Destinations.Add(Oscil[0].GetModDestGain()); 
				return;

			case ESynth1PatchDestination::Osc1Frequency:
				Destinations.Add(Oscil[0].GetModDestFrequency());
				return;

			case ESynth1PatchDestination::Osc1Pulsewidth:
				Destinations.Add(Oscil[0].GetModDestPulseWidth());
				return;

			case ESynth1PatchDestination::Osc2Gain:
				Destinations.Add(Oscil[1].GetModDestGain());
				return;

			case ESynth1PatchDestination::Osc2Frequency:
				Destinations.Add(Oscil[1].GetModDestFrequency());
				return;

			case ESynth1PatchDestination::Osc2Pulsewidth:
				Destinations.Add(Oscil[1].GetModDestPulseWidth());
				return;

			case ESynth1PatchDestination::FilterFrequency:
				Destinations.Add(OnePoleFilter.GetModDestCutoffFrequency());
				Destinations.Add(StateVarFilter.GetModDestCutoffFrequency());
				Destinations.Add(LadderFilter.GetModDestCutoffFrequency());
				return;

			case ESynth1PatchDestination::FilterQ:
				Destinations.Add(OnePoleFilter.GetModDestQ());
				Destinations.Add(StateVarFilter.GetModDestQ());
				Destinations.Add(LadderFilter.GetModDestQ());
				return;

			case ESynth1PatchDestination::Gain:
				Destinations.Add(Amp.GetModDestGainScale());
				return;

			case ESynth1PatchDestination::Pan:
				Destinations.Add(Amp.GetModDestPan());
				return;

			case ESynth1PatchDestination::LFO1Frequency:
				Destinations.Add(LFO[0].GetModDestFrequency());
				return;

			case ESynth1PatchDestination::LFO1Gain:
				Destinations.Add(LFO[0].GetModDestGain());
				return;

			case ESynth1PatchDestination::LFO2Frequency:
				Destinations.Add(LFO[1].GetModDestFrequency());
				return;

			case ESynth1PatchDestination::LFO2Gain:
				Destinations.Add(LFO[1].GetModDestGain());
				return;

			default:
				return;
		}
	}

	void FEpicSynth1Voice::ClearPatches()
	{
		// Clear dynamic patches
		FModulationMatrix* ModMatrix = &ParentSynth->ModMatrix;
		for (auto PatchEntry : DynamicPatches)
		{
			TSharedPtr<FPatch> NewPatch = PatchEntry.Value;
			ModMatrix->RemovePatch(VoiceId, NewPatch.Get());
		}

		DynamicPatches.Reset();
	}

	bool FEpicSynth1Voice::CreatePatch(const FPatchId PatchId, const ESynth1PatchSource PatchSource, const TArray<FSynth1PatchCable>& PatchCables, const bool bEnableByDefault)
	{
		FModulationMatrix& ModMatrix = ParentSynth->ModMatrix;
		if (DynamicPatches.Contains(PatchId.Id))
		{
			return false;
		}
		
		TSharedPtr<FPatch> NewPatch = TSharedPtr<FPatch>(new FPatch);
		NewPatch->bEnabled = bEnableByDefault;
		NewPatch->Source = GetPatchSource(PatchSource);
		for (int32 i = 0; i < PatchCables.Num(); ++i)
		{
			TArray<FPatchDestination> Destinations;
			GetPatchDestinations(PatchCables[i].Destination, Destinations);

			for (int32 j = 0; j < Destinations.Num(); ++j)
			{
				Destinations[j].Depth = PatchCables[i].Depth;
			}

			NewPatch->Destinations.Append(Destinations);
		}
		DynamicPatches.Add(PatchId.Id, NewPatch);

		ModMatrix.AddPatch(VoiceId, NewPatch.Get());

		return true;
	}

	bool FEpicSynth1Voice::SetEnablePatch(const FPatchId PatchId, const bool bIsEnabled)
	{
		TSharedPtr<FPatch>* Patch = DynamicPatches.Find(PatchId.Id);
		if (Patch)
		{
			(*Patch)->bEnabled = bIsEnabled;
			return true;
		}
		return false;
	}

	void FEpicSynth1Voice::Reset()
	{
		bIsFinished = true;
		bIsActive = false;
		VoiceGeneration = INDEX_NONE;

		for (int32 i = 0; i < NumOscillators; ++i)
		{
			Oscil[i].Reset();
			OscilPan[i].Reset();
		}

		OnePoleFilter.Reset();
		StateVarFilter.Reset();
		LadderFilter.Reset();

		Amp.Reset();
	}

	void FEpicSynth1Voice::NoteOn(const uint32 InMidiNote, const float InVelocity, const float InDurationSec)
	{
		bIsActive = true;
		bIsFinished = false;
		ControlSampleCount = 0;
		CurrentSampleCount = 0;

		// Reset sample count to -1.0f and set if passed in a valid duration value
		DurationSampleCount = -1;
		if (InDurationSec > 0.0f)
		{
			DurationSampleCount = (int32)(InDurationSec * ParentSynth->SampleRate);
		}

		// Set the voice generation of the voice and bump the count
		VoiceGeneration = ParentSynth->VoiceGeneration++;

		// Set the voice lerp immediately to the last note the parent synth played
		const float StartingFrequency = GetFrequencyFromMidi((float)ParentSynth->LastMidiNote);
		const float EndingFrequency = GetFrequencyFromMidi((float)InMidiNote);

		PortamentoFrequency.SetValueRange(StartingFrequency, EndingFrequency, ParentSynth->Portamento);

		// Start the oscillators playing if they're not already
		if (!Oscil[0].IsPlaying())
		{
			Amp.Reset();

			Amp.SetVelocity(InVelocity);

			for (int32 i = 0; i < NumOscillators; ++i)
			{
				Oscil[i].Start();
			}
		}
		else if (GainEnv.IsRetrigger())
		{
			Amp.SetVelocity(InVelocity);
		}

		// Start the LFOs
		for (int32 i = 0; i < NumLFOs; ++i)
		{
			LFO[i].Start();
		}

		// Start the envelope
		GainEnv.Start();
		ModEnv.Start();

		// Store the midi note
		MidiNote = InMidiNote;
	}

	void FEpicSynth1Voice::NoteOff(const uint32 InMidiNote, const bool bAllNotesOff)
	{
		// No longer need to worry about duration
		DurationSampleCount = -1;

		if (!IsFinished() && (MidiNote == InMidiNote || bAllNotesOff)) 
		{
			GainEnv.Stop();
			ModEnv.Stop();

			if (GainEnv.IsDone())
			{
				bIsFinished = true;
				Amp.Reset();
			}
		}
	}

	void FEpicSynth1Voice::Kill()
	{
		for (int32 i = 0; i < NumOscillators; ++i)
		{
			Oscil[i].Stop();
		}

		for (int32 i = 0; i < NumLFOs; ++i)
		{
			LFO[i].Stop();
		}

		GainEnv.Kill();
		ModEnv.Kill();
		Amp.Reset();
		bIsActive = false;
		bIsFinished = true;
		VoiceGeneration = INDEX_NONE;
	}

	void FEpicSynth1Voice::Shutdown()
	{
		GainEnv.Shutdown();
		ModEnv.Shutdown();
		Amp.Reset();
		VoiceGeneration = INDEX_NONE;
	}

	void FEpicSynth1Voice::SetLFOPatch(const int32 InLFOIndex, const ESynthLFOPatchType InPatchType)
	{
		for (int32 i = 0; i < (int32)ESynthLFOPatchType::Count; ++i)
		{
			LFOPatches[InLFOIndex][i].bEnabled = false;
		}

		if (InPatchType != ESynthLFOPatchType::PatchToNone)
		{
			LFOPatches[InLFOIndex][(int32)InPatchType].bEnabled = true;
		}
		CurrentPatchType[InLFOIndex] = InPatchType;
	}

	void FEpicSynth1Voice::SetEnvModPatch(const ESynthModEnvPatch InPatchType)
	{
		for (int32 i = 0; i < (int32)ESynthModEnvPatch::Count; ++i)
		{
			ModEnvPatches[i].bEnabled = false;
		}

		if (InPatchType != ESynthModEnvPatch::PatchToNone)
		{
			ModEnvPatches[(int32)InPatchType].bEnabled = true;
		}
		CurrentModPatchType = InPatchType;
	}

	void FEpicSynth1Voice::SetEnvModBiasPatch(const ESynthModEnvBiasPatch InPatchType)
	{
		for (int32 i = 0; i < (int32)ESynthModEnvBiasPatch::Count; ++i)
		{
			ModEnvBiasPatches[i].bEnabled = false;
		}

		if (InPatchType != ESynthModEnvBiasPatch::PatchToNone)
		{
			ModEnvBiasPatches[(int32)InPatchType].bEnabled = true;
		}
		CurrentModBiasPatchType = InPatchType;
	}

	void FEpicSynth1Voice::Generate(float OutSamples[2])
	{
		if (GainEnv.IsDone())
		{
			ModEnv.Kill();
			bIsFinished = true;
			return;
		}

		for (int32 i = 0; i < 2; ++i)
		{
			Oscil[i].SetFrequency(PortamentoFrequency.GetNextValue());
		}

		FModulationMatrix* ModMatrix = &ParentSynth->ModMatrix;

		// Update control block at reduced sample rate
		ControlSampleCount = ControlSampleCount & (ParentSynth->ControlSamplePeriod - 1);
		if (ControlSampleCount == 0)
		{
			GainEnv.Generate();
			ModEnv.Generate();

			ModMatrix->Update(VoiceId, 0);

			LFO[0].Update();
			LFO[0].Generate();

			LFO[1].Update();
			LFO[1].Generate();

			ModMatrix->Update(VoiceId, 1);

			Oscil[0].Update();
			Oscil[1].Update();

			OscilPan[0].Update();
			OscilPan[1].Update();
			Amp.Update();
		}

		++ControlSampleCount;

#if !SYNTH_DEBUG_MODE
		CurrentFilter->Update();
#endif

		// Compute the left and right output
#if 0
		float SampleInput = 0.0f;
		for (int32 i = 0; i < 2; ++i)
		{
			SampleInput += 0.5f * Oscil[i].Generate();
		}

		Amp.GetGain(SampleInput, &OutSamples[0], &OutSamples[1]);
#else
		if (ParentSynth->bIsUnison)
		{
			float SampleInput = 0.0f;
			for (int32 i = 0; i < 2; ++i)
			{
				SampleInput += 0.5f * Oscil[i].Generate();
			}

			Amp.ProcessAudio(SampleInput, &OutSamples[0], &OutSamples[1]);
		}
		else
		{
			/// Compute the stereo spread of the oscillators

			// Compute the left/right input samples
			float LeftSampleInput = 0.5f * Oscil[0].Generate();
			float RightSampleInput = 0.5f * Oscil[1].Generate();

			// Compute the left channel panned sample
			float LeftOscLeftOut = 0.0f;
			float LeftOscRightOut = 0.0f;
			OscilPan[0].ProcessAudio(LeftSampleInput, &LeftOscLeftOut, &LeftOscRightOut);

			// Compute the right channel panned sample
			float RightOscLeftOut = 0.0f;
			float RightOscRightOut = 0.0f;
			OscilPan[1].ProcessAudio(RightSampleInput, &RightOscLeftOut, &RightOscRightOut);

			// Mix the left/right channel panned back together to one stereo frame
			LeftSampleInput = LeftOscLeftOut + RightOscLeftOut;
			RightSampleInput = LeftOscRightOut + RightOscRightOut;

			// Perform the normal stereo pan
			Amp.ProcessAudio(LeftSampleInput, RightSampleInput, &OutSamples[0], &OutSamples[1]);
		}
#endif

		// Apply filters...
#if !SYNTH_DEBUG_MODE
		CurrentFilter->ProcessAudioFrame(OutSamples, OutSamples);
#endif
		// Stop the oscillators if they're done
		if (GainEnv.IsDone())
		{
			ModEnv.Kill();

			Oscil[0].Stop();
			Oscil[1].Stop();

			LFO[0].Stop();
			LFO[1].Stop();

			// We naturally finished!
			bIsFinished = true;
		}

		// Check if we're automatically turning ourselves off
		if (DurationSampleCount > 0)
		{
			++CurrentSampleCount;
			if (CurrentSampleCount > DurationSampleCount)
			{

				NoteOff(MidiNote);
				DurationSampleCount = -1;
			}
		}
	}

	FEpicSynth1::FEpicSynth1()
		: MaxNumVoices(1)
		, NumVoices(MaxNumVoices)
		, NumActiveVoices(0)
		, NumStoppingVoices(8)
		, LastVoice(nullptr)
		, SampleRate(44100.0f)
		, ControlSampleRate(0.0f)
		, ControlSamplePeriod(256)
		, Portamento(0.0f)
		, LastMidiNote(0)
		, VoiceGeneration(0)
		, BaseFilterFreq(0.5f*SampleRate)
		, FilterFreqMod(0.0f)
		, BaseFilterQ(1.5f)
		, FilterQMod(0.0f)
		, FilterType(EFilter::LowPass)
		, FilterAlgorithm(ESynthFilterAlgorithm::OnePole)
		, bIsUnison(false)
		, bIsStereoEnabled(true)
		, bIsChorusEnabled(true)
	{
	}

	FEpicSynth1::~FEpicSynth1()
	{
		for (int32 i = 0; i < Voices.Num(); ++i)
		{
			delete Voices[i];
			Voices[i] = nullptr;
		}
	}

	void FEpicSynth1::Init(const float InSampleRate, const int32 InNumVoices)
	{
		// Always have 1 more than the voices requested for a stopping voice and always at least 1
#if SYNTH_DEBUG_MODE
		MaxNumVoices = 4;
#else
		MaxNumVoices = FMath::Max(InNumVoices, 1);
#endif
		// Default the synth to mono mode
		NumVoices = 1;
		SampleRate = InSampleRate;

		ModMatrix.Init(MaxNumVoices + NumStoppingVoices);

		ControlSampleRate = SampleRate / ControlSamplePeriod;

		for (int32 VoiceId = 0; VoiceId < MaxNumVoices + NumStoppingVoices; ++VoiceId)
		{
			FreeVoices.Push(VoiceId);

			const int32 Index = Voices.Add(new FEpicSynth1Voice());
			FEpicSynth1Voice* NewVoice = Voices[Index];
			NewVoice->Init(this, VoiceId);
		}

		StereoDelay.Init(SampleRate, 2.0f);
		Chorus.Init(InSampleRate, 2.0f);
	}

	void FEpicSynth1::SetMonoMode(const bool bInIsMonoMode)
	{
		if (bInIsMonoMode)
		{
			NumVoices = 1;

			StopAllVoicesExceptNewest();
		}
		else
		{
			NumVoices = MaxNumVoices;
		}
	}

	void FEpicSynth1::StopAllVoicesExceptNewest()
	{
		const uint32 LastGeneration = VoiceGeneration - 1;
		for (int32 VoiceId = 0; VoiceId < Voices.Num(); ++VoiceId)
		{
			if (Voices[VoiceId]->IsActive() && Voices[VoiceId]->GetGeneration() != LastGeneration)
			{
				Voices[VoiceId]->Kill();
				FreeVoices.Push(VoiceId);
			}
		}
	}

	void FEpicSynth1::NoteOn(const uint32 InMidiNote, const float InVelocity, const float Duration)
	{
		FEpicSynth1Voice* Voice = nullptr;

		// Special mono-synth case, just reuse!
		if (NumVoices == 1)
		{
			if (LastVoice)
			{
				Voice = LastVoice;
			}
			else
			{
				const int32 VoiceIndex = FreeVoices.Pop();
				Voice = Voices[VoiceIndex];
				LastVoice = Voice;
				++NumActiveVoices;
			}
		}
		else
		{
			const int32 VoicesLeft = MaxNumVoices - NumActiveVoices;
			if (VoicesLeft >= FreeVoices.Num())
			{
				UE_LOGF(LogSynthesis, Warning, "Not enough voices left.");
				return;
			}

			int32 VoiceId = INDEX_NONE;
			if (VoicesLeft > 0)
			{
				VoiceId = FreeVoices.Pop();
				Voice = Voices[VoiceId];
			}
			else if (FreeVoices.Num() > 0)
			{
				uint32 OldestId = GetOldestPlayingId();
				if (OldestId < (uint32)Voices.Num())
				{
					Voices[OldestId]->Shutdown();
				}
				VoiceId = FreeVoices.Pop();
				Voice = Voices[VoiceId];
			}
			else
			{
				uint32 OldestId = GetOldestPlayingId();
				if (OldestId < (uint32)Voices.Num())
				{
					Voices[OldestId]->Kill();
					--NumActiveVoices;

					VoiceId = OldestId;
					Voice = Voices[OldestId];
				}
			}

			// Fallback to grabbing the 0th voice
			if (VoiceId == INDEX_NONE && Voices.Num() > 0)
			{
				Voices[0]->Kill();
				VoiceId = 0;
				Voice = Voices[0];
			}

			++NumActiveVoices;
		}


#if SYNTH_DEBUG_MODE
		Voice->NoteOn(InMidiNote, 100, Duration);
#else
		Voice->NoteOn(InMidiNote, InVelocity, Duration);
#endif

		LastMidiNote = InMidiNote;
	}

	void FEpicSynth1::NoteOff(const uint32 InMidiNote, const bool bAllNotesOff, const bool bKillAllNotes)
	{
		// Loop through voices and call note off... only notes which match the midi note will actually turn off.
		for (int32 VoiceId = 0; VoiceId < Voices.Num(); ++VoiceId)
		{
			if (bKillAllNotes)
			{
				Voices[VoiceId]->Kill();
			}
			else if (bAllNotesOff)
			{
				Voices[VoiceId]->NoteOff(InMidiNote, bAllNotesOff);
			}
			else
			{
				Voices[VoiceId]->NoteOff(InMidiNote);
			}
		}
	}

	void FEpicSynth1::SetOscType(const int32 InOscIndex, const EOsc::Type InOscType)
	{
		if (InOscIndex < FEpicSynth1Voice::NumOscillators)
		{
			for (int32 VoiceId = 0; VoiceId < Voices.Num(); ++VoiceId)
			{
				Voices[VoiceId]->Oscil[InOscIndex].SetType(InOscType);
			}
		}
	}

	void FEpicSynth1::SetOscGain(const int32 InOscIndex, const float InGain)
	{
		if (InOscIndex < FEpicSynth1Voice::NumOscillators)
		{
			for (int32 VoiceId = 0; VoiceId < Voices.Num(); ++VoiceId)
			{
				Voices[VoiceId]->Oscil[InOscIndex].SetGain(InGain);
			}
		}
	}

	void FEpicSynth1::SetOscGainMod(const int32 InOscIndex, const float InGainMod)
	{
		if (InOscIndex < FEpicSynth1Voice::NumOscillators)
		{
			for (int32 VoiceId = 0; VoiceId < Voices.Num(); ++VoiceId)
			{
				Voices[VoiceId]->Oscil[InOscIndex].SetGainMod(InGainMod);
			}
		}
	}

	void FEpicSynth1::SetOscDetune(const int32 InOscIndex, const float InDetuneFreq)
	{
		if (InOscIndex < FEpicSynth1Voice::NumOscillators)
		{
			for (int32 VoiceId = 0; VoiceId < Voices.Num(); ++VoiceId)
			{
				Voices[VoiceId]->Oscil[InOscIndex].SetDetune(InDetuneFreq);
			}
		}
	}

	void FEpicSynth1::SetOscOctave(const int32 InOscIndex, const float InOctave)
	{
		if (InOscIndex < FEpicSynth1Voice::NumOscillators)
		{
			for (int32 VoiceId = 0; VoiceId < Voices.Num(); ++VoiceId)
			{
				Voices[VoiceId]->Oscil[InOscIndex].SetOctave(InOctave);
			}
		}
	}

	void FEpicSynth1::SetOscSemitones(const int32 InOscIndex, const float InSemitones)
	{
		if (InOscIndex < FEpicSynth1Voice::NumOscillators)
		{
			for (int32 VoiceId = 0; VoiceId < Voices.Num(); ++VoiceId)
			{
				Voices[VoiceId]->Oscil[InOscIndex].SetSemitones(InSemitones);
			}
		}
	}

	void FEpicSynth1::SetOscCents(const int32 InOscIndex, const float InCents)
	{
		if (InOscIndex < FEpicSynth1Voice::NumOscillators)
		{
			for (int32 VoiceId = 0; VoiceId < Voices.Num(); ++VoiceId)
			{
				Voices[VoiceId]->Oscil[InOscIndex].SetCents(InCents);
			}
		}
	}

	void FEpicSynth1::SetOscPitchBend(const int32 InOscIndex, const float InPitchBend)
	{
		if (InOscIndex < FEpicSynth1Voice::NumOscillators)
		{
			for (int32 VoiceId = 0; VoiceId < Voices.Num(); ++VoiceId)
			{
				Voices[VoiceId]->Oscil[InOscIndex].SetPitchBend(InPitchBend);
			}
		}
	}

	void FEpicSynth1::SetOscPortamento(const float InPortamento)
	{
		Portamento = FMath::Clamp(InPortamento, 0.0f, 1.0f);
	}

	void FEpicSynth1::SetOscPulseWidth(const int32 InOscIndex, const float InPulseWidth)
	{
		if (InOscIndex < FEpicSynth1Voice::NumOscillators)
		{
			for (int32 VoiceId = 0; VoiceId < Voices.Num(); ++VoiceId)
			{
				Voices[VoiceId]->Oscil[InOscIndex].SetPulseWidth(FMath::Clamp(InPulseWidth, 0.0f, 1.0f));
			}
		}
	}

	void FEpicSynth1::SetOscSpread(const float InSpread)
	{
		const float Spread = FMath::Clamp(InSpread, -1.0f, 1.0f);
		for (int32 VoiceId = 0; VoiceId < Voices.Num(); ++VoiceId)
		{
			Voices[VoiceId]->OscilPan[0].SetPan(-InSpread);
			Voices[VoiceId]->OscilPan[1].SetPan(InSpread);
		}
	}

	void FEpicSynth1::SetOscUnison(const bool bInUnison)
	{
		bIsUnison = bInUnison;
	}

	void FEpicSynth1::SetOscSync(const bool bIsSync)
	{
		for (int32 VoiceId = 0; VoiceId < Voices.Num(); ++VoiceId)
		{
			Voices[VoiceId]->Oscil[1].SetSync(bIsSync);
		}
	}

	void FEpicSynth1::SetLFOType(const int32 LFOIndex, const ELFO::Type InLFOType)
	{
		if (LFOIndex < FEpicSynth1Voice::NumLFOs)
		{
			for (int32 VoiceId = 0; VoiceId < Voices.Num(); ++VoiceId)
			{
				Voices[VoiceId]->LFO[LFOIndex].SetType(InLFOType);
			}
		}
	}

	void FEpicSynth1::SetLFOMode(const int32 LFOIndex, const ELFOMode::Type InLFOMode)
	{
		if (LFOIndex < FEpicSynth1Voice::NumLFOs)
		{
			for (int32 VoiceId = 0; VoiceId < Voices.Num(); ++VoiceId)
			{
				Voices[VoiceId]->LFO[LFOIndex].SetMode(InLFOMode);
			}
		}
	}

	void FEpicSynth1::SetLFOPatch(const int32 LFOIndex, const ESynthLFOPatchType InPatchType)
	{
		bool bSetPatch = false;
		if ((InPatchType == ESynthLFOPatchType::PatchLFO1ToLFO2Frequency || InPatchType == ESynthLFOPatchType::PatchLFO1ToLFO2Gain))
		{
			if (LFOIndex == 0)
			{
				bSetPatch = true;
			}
		}
		else
		{
			bSetPatch = true;
		}

		if (bSetPatch)
		{
			if (LFOIndex < FEpicSynth1Voice::NumLFOs)
			{
				for (int32 VoiceId = 0; VoiceId < Voices.Num(); ++VoiceId)
				{
					Voices[VoiceId]->SetLFOPatch(LFOIndex, InPatchType);
				}
			}
		}
	}

	void FEpicSynth1::SetLFOGain(const int32 LFOIndex, const float InLFOGain)
	{
		if (LFOIndex < FEpicSynth1Voice::NumLFOs)
		{
			for (int32 VoiceId = 0; VoiceId < Voices.Num(); ++VoiceId)
			{
				Voices[VoiceId]->LFO[LFOIndex].SetGain(FMath::Clamp(InLFOGain, 0.0f, 1.0f));
			}
		}
	}

	void FEpicSynth1::SetLFOGainMod(const int32 LFOIndex, const float InLFOGainMod)
	{
		if (LFOIndex < FEpicSynth1Voice::NumLFOs)
		{
			for (int32 VoiceId = 0; VoiceId < Voices.Num(); ++VoiceId)
			{
				Voices[VoiceId]->LFO[LFOIndex].SetGainMod(FMath::Clamp(InLFOGainMod, 0.0f, 1.0f));
			}
		}
	}

	void FEpicSynth1::SetLFOFrequency(const int32 LFOIndex, const float InLFOFrequency)
	{
		if (LFOIndex < FEpicSynth1Voice::NumLFOs)
		{
			for (int32 VoiceId = 0; VoiceId < Voices.Num(); ++VoiceId)
			{
				Voices[VoiceId]->LFO[LFOIndex].SetFrequency(InLFOFrequency);
			}
		}
	}

	void FEpicSynth1::SetLFOFrequencyMod(const int32 LFOIndex, const float InLFOFrequencyMod)
	{
		if (LFOIndex < FEpicSynth1Voice::NumLFOs)
		{
			for (int32 VoiceId = 0; VoiceId < Voices.Num(); ++VoiceId)
			{
				Voices[VoiceId]->LFO[LFOIndex].SetFrequencyMod(InLFOFrequencyMod);
			}
		}
	}

	void FEpicSynth1::SetLFOPulseWidth(const int32 LFOIndex, const float InPulseWidth)
	{
		if (LFOIndex < FEpicSynth1Voice::NumLFOs)
		{
			for (int32 VoiceId = 0; VoiceId < Voices.Num(); ++VoiceId)
			{
				Voices[VoiceId]->LFO[LFOIndex].SetPulseWidth(InPulseWidth);
			}
		}
	}

	void FEpicSynth1::SetFilterAlgorithm(const ESynthFilterAlgorithm InFilterAlgorithm)
	{
		FilterAlgorithm = InFilterAlgorithm;
		SwitchFilter();
	}

	void FEpicSynth1::SetFilterType(const EFilter::Type InFilterType)
	{
		FilterType = InFilterType;
		for (int32 VoiceId = 0; VoiceId < Voices.Num(); ++VoiceId)
		{
			Voices[VoiceId]->CurrentFilter->SetFilterType(InFilterType);
		}
	}

	void FEpicSynth1::SetFilterFrequency(const float InFilterFrequency)
	{
		BaseFilterFreq = InFilterFrequency;
		for (int32 VoiceId = 0; VoiceId < Voices.Num(); ++VoiceId)
		{
			Voices[VoiceId]->CurrentFilter->SetFrequency(InFilterFrequency);
		}
	}

	void FEpicSynth1::SetFilterFrequencyMod(const float InFilterFrequencyMod)
	{
		FilterFreqMod = InFilterFrequencyMod;
		for (int32 VoiceId = 0; VoiceId < Voices.Num(); ++VoiceId)
		{		
			Voices[VoiceId]->CurrentFilter->SetFrequencyMod(InFilterFrequencyMod);
		}
	}

	void FEpicSynth1::SetFilterQ(const float InFilterQ)
	{
		BaseFilterQ = InFilterQ;
		for (int32 VoiceId = 0; VoiceId < Voices.Num(); ++VoiceId)
		{
			Voices[VoiceId]->CurrentFilter->SetQ(InFilterQ);
		}
	}

	void FEpicSynth1::SetFilterQMod(const float InFilterQMod)
	{
		FilterQMod = InFilterQMod;
		for (int32 VoiceId = 0; VoiceId < Voices.Num(); ++VoiceId)
		{
			Voices[VoiceId]->CurrentFilter->SetQ(InFilterQMod);
		}
	}

	uint32 FEpicSynth1::GetOldestPlayingId()
	{
		// Find the playing voice with the lowest voice generation (oldest id)
		uint32 OldestId = INDEX_NONE;
		uint32 OldestGeneration = INDEX_NONE;
		for (int32 VoiceId = 0; VoiceId < Voices.Num(); ++VoiceId)
		{
			uint32 ThisVoiceGeneration = Voices[VoiceId]->GetGeneration();
			if (ThisVoiceGeneration < OldestGeneration)
			{
				OldestGeneration = ThisVoiceGeneration;
				OldestId = VoiceId;
				check(OldestId < (uint32)Voices.Num());
			}
		}
		return OldestId;
	}

	void FEpicSynth1::SwitchFilter()
	{
		for (int32 VoiceId = 0; VoiceId < Voices.Num(); ++VoiceId)
		{
			IFilter* CurrentFilter = nullptr;
			
			switch (FilterAlgorithm)
			{
				case ESynthFilterAlgorithm::OnePole:
				{
					CurrentFilter = &Voices[VoiceId]->OnePoleFilter;
				}
				break;

				case ESynthFilterAlgorithm::StateVariable:
				{
					CurrentFilter = &Voices[VoiceId]->StateVarFilter;
				}
				break;

				case ESynthFilterAlgorithm::Ladder:
				{
					CurrentFilter = &Voices[VoiceId]->LadderFilter;
				}
				break;
			}

			CurrentFilter->SetFilterType(FilterType);
			CurrentFilter->SetFrequency(BaseFilterFreq);
			CurrentFilter->SetFrequencyMod(FilterFreqMod);
			CurrentFilter->SetQ(BaseFilterQ);
			CurrentFilter->SetQMod(FilterQMod);
			Voices[VoiceId]->CurrentFilter = CurrentFilter;
		}
	}

	void FEpicSynth1::SetEnvAttackTime(const float InAttackTimeMsec)
	{
		for (int32 VoiceId = 0; VoiceId < Voices.Num(); ++VoiceId)
		{
			Voices[VoiceId]->GainEnv.SetAttackTime(InAttackTimeMsec);
		}
	}

	void FEpicSynth1::SetEnvDecayTime(const float InDecayTimeMsec)
	{
		for (int32 VoiceId = 0; VoiceId < Voices.Num(); ++VoiceId)
		{
			Voices[VoiceId]->GainEnv.SetDecayTime(InDecayTimeMsec);
		}
	}

	void FEpicSynth1::SetEnvSustainGain(const float InSustainGain)
	{
		for (int32 VoiceId = 0; VoiceId < Voices.Num(); ++VoiceId)
		{
			Voices[VoiceId]->GainEnv.SetSustainGain(InSustainGain);
		}
	}

	void FEpicSynth1::SetEnvReleaseTime(const float InReleaseTimeMsec)
	{
		for (int32 VoiceId = 0; VoiceId < Voices.Num(); ++VoiceId)
		{
			Voices[VoiceId]->GainEnv.SetReleaseTime(InReleaseTimeMsec);
		}
	}

	void FEpicSynth1::SetEnvLegatoEnabled(const bool bIsLegatoEnable)
	{
		for (int32 VoiceId = 0; VoiceId < Voices.Num(); ++VoiceId)
		{
			Voices[VoiceId]->GainEnv.SetLegato(bIsLegatoEnable);
			Voices[VoiceId]->ModEnv.SetLegato(bIsLegatoEnable);
		}
	}

	void FEpicSynth1::SetEnvRetriggerMode(const bool bIsRetriggerMode)
	{
		for (int32 VoiceId = 0; VoiceId < Voices.Num(); ++VoiceId)
		{
			Voices[VoiceId]->GainEnv.SetRetrigger(bIsRetriggerMode);
			Voices[VoiceId]->ModEnv.SetRetrigger(bIsRetriggerMode);
		}
	}

	void FEpicSynth1::SetModEnvPatch(const ESynthModEnvPatch InPatchType)
	{
		for (int32 VoiceId = 0; VoiceId < Voices.Num(); ++VoiceId)
		{
			Voices[VoiceId]->SetEnvModPatch(InPatchType);
		}
	}

	void FEpicSynth1::SetModEnvBiasPatch(const ESynthModEnvBiasPatch InPatchType)
	{
		for (int32 VoiceId = 0; VoiceId < Voices.Num(); ++VoiceId)
		{
			Voices[VoiceId]->SetEnvModBiasPatch(InPatchType);
		}
	}

	void FEpicSynth1::SetModEnvInvert(const bool bInInvert)
	{
		for (int32 VoiceId = 0; VoiceId < Voices.Num(); ++VoiceId)
		{
			Voices[VoiceId]->ModEnv.SetInvert(bInInvert);
		}
	}

	void FEpicSynth1::SetModEnvBiasInvert(const bool bInInvert)
	{
		for (int32 VoiceId = 0; VoiceId < Voices.Num(); ++VoiceId)
		{
			Voices[VoiceId]->ModEnv.SetBiasInvert(bInInvert);
		}
	}

	void FEpicSynth1::SetModEnvDepth(const float InDepth)
	{
		for (int32 VoiceId = 0; VoiceId < Voices.Num(); ++VoiceId)
		{
			Voices[VoiceId]->ModEnv.SetDepth(InDepth);
		}
	}

	void FEpicSynth1::SetModEnvAttackTime(const float InAttackTimeMsec)
	{
		for (int32 VoiceId = 0; VoiceId < Voices.Num(); ++VoiceId)
		{
			Voices[VoiceId]->ModEnv.SetAttackTime(InAttackTimeMsec);
		}
	}

	void FEpicSynth1::SetModEnvDecayTime(const float InDecayTimeMsec)
	{
		for (int32 VoiceId = 0; VoiceId < Voices.Num(); ++VoiceId)
		{
			Voices[VoiceId]->ModEnv.SetDecayTime(InDecayTimeMsec);
		}
	}

	void FEpicSynth1::SetModEnvSustainGain(const float InSustainGain)
	{
		for (int32 VoiceId = 0; VoiceId < Voices.Num(); ++VoiceId)
		{
			Voices[VoiceId]->ModEnv.SetSustainGain(InSustainGain);
		}
	}

	void FEpicSynth1::SetModEnvReleaseTime(const float InReleaseTimeMsec)
	{
		for (int32 VoiceId = 0; VoiceId < Voices.Num(); ++VoiceId)
		{
			Voices[VoiceId]->ModEnv.SetReleaseTime(InReleaseTimeMsec);
		}
	}


	void FEpicSynth1::SetPan(const float InPan)
	{
		for (int32 VoiceId = 0; VoiceId < Voices.Num(); ++VoiceId)
		{
			Voices[VoiceId]->Amp.SetPan(InPan);
		}
	}

	void FEpicSynth1::SetGainDb(const float InGainDb)
	{
		for (int32 VoiceId = 0; VoiceId < Voices.Num(); ++VoiceId)
		{
			Voices[VoiceId]->Amp.SetGainDb(InGainDb);
		}
	}

	void FEpicSynth1::SetStereoDelayIsEnabled(const bool bInIsStereoEnabled)
	{
		bIsStereoEnabled = bInIsStereoEnabled;
	}

	void FEpicSynth1::SetStereoDelayMode(EStereoDelayMode::Type InStereoDelayMode)
	{
		StereoDelay.SetMode(InStereoDelayMode);
	}
	
	void FEpicSynth1::SetStereoDelayTimeMsec(const float InDelayTimeMsec) 
	{ 
		StereoDelay.SetDelayTimeMsec(InDelayTimeMsec); 
	}
	
	void FEpicSynth1::SetStereoDelayFeedback(const float InDelayFeedback)
	{ 
		StereoDelay.SetFeedback(InDelayFeedback); 
	}
	
	void FEpicSynth1::SetStereoDelayRatio(const float InDelayRatio) 
	{ 
		StereoDelay.SetDelayRatio(InDelayRatio); 
	}
	
	void FEpicSynth1::SetStereoDelayWetLevel(const float InDelayWetLevel)
	{
		StereoDelay.SetWetLevel(InDelayWetLevel);
	}

	void FEpicSynth1::SetChorusEnabled(const bool bInIsChorusEnabled)
	{
		bIsChorusEnabled = bInIsChorusEnabled;
	}

	void FEpicSynth1::SetChorusDepth(const EChorusDelays::Type InType, const float InDepth)
	{
		Chorus.SetDepth(InType, InDepth);
	}

	void FEpicSynth1::SetChorusFeedback(const EChorusDelays::Type InType, const float InFeedback)
	{
		Chorus.SetFeedback(InType, InFeedback);
	}

	void FEpicSynth1::SetChorusFrequency(const EChorusDelays::Type InType, const float InFrequency)
	{
		Chorus.SetFrequency(InType, InFrequency);
	}

	void FEpicSynth1::ClearPatches()
	{
		ModMatrix.ResetPatchSourceState();

		for (int32 VoiceId = 0; VoiceId < Voices.Num(); ++VoiceId)
		{
			Voices[VoiceId]->Reset();
			Voices[VoiceId]->ClearPatches();
		}
	}

	FPatchId FEpicSynth1::CreatePatch(const ESynth1PatchSource PatchSource, const TArray<FSynth1PatchCable>& PatchCables, const bool bEnableByDefault)
	{
		static int32 PatchCount = 0;
		FPatchId NewPatchId;
		NewPatchId.Id = PatchCount++;

		for (int32 VoiceId = 0; VoiceId < Voices.Num(); ++VoiceId)
		{
			bool bSuccess = Voices[VoiceId]->CreatePatch(NewPatchId, PatchSource, PatchCables, bEnableByDefault);
			ensure(bSuccess);
		}

		return NewPatchId;
	}

	bool FEpicSynth1::SetEnablePatch(const FPatchId PatchId, bool bIsEnabled)
	{
		for (int32 VoiceId = 0; VoiceId < Voices.Num(); ++VoiceId)
		{
			if (!Voices[VoiceId]->SetEnablePatch(PatchId, bIsEnabled))
			{
				return false;
			}
		}
		return true;
	}

	void FEpicSynth1::GenerateFrame(float* OutFrame)
	{
		OutFrame[0] = 0.0f;
		OutFrame[1] = 0.0f;

		for (int32 VoiceId = 0; VoiceId < Voices.Num(); ++VoiceId)
		{
			// Don't process voice if its finished (i.e. finished envelope)
			if (Voices[VoiceId]->IsFinished())
			{
				// If it's still active, then reset it and claim the voice id for reuse
				if (Voices[VoiceId]->IsActive())
				{
					--NumActiveVoices;
					Voices[VoiceId]->Reset();

					if (NumVoices != 1)
					{
#if SYNTH_DEBUG_MODE
						bool bIsUnique = true;
						for (int32 i = 0; i < FreeVoices.Num(); ++i)
						{
							if (FreeVoices[i] == VoiceId)
							{
								bIsUnique = false;
							}
						}
						checkf(bIsUnique, TEXT("Freeing voice id is not unique"));
#endif
						FreeVoices.Push(VoiceId);
						checkf(FreeVoices.Num() <= MaxNumVoices + NumStoppingVoices, TEXT("Invalid free voice size."));
					}
				}

				continue;
			}

			float VoiceSamples[2] = { 0.0f, 0.0f };
			Voices[VoiceId]->Generate(VoiceSamples);

			// Mix the voices audio with the output
			OutFrame[0] += VoiceSamples[0];
			OutFrame[1] += VoiceSamples[1];
		}

		if (bIsChorusEnabled)
		{
			Chorus.ProcessAudioFrame(OutFrame, OutFrame);
		}

		if (bIsStereoEnabled)
		{
			StereoDelay.ProcessAudioFrame(OutFrame, OutFrame);
		}
	}


}
