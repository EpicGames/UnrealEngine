// Copyright Epic Games, Inc. All Rights Reserved.

#include "SynthComponents/EpicSynth1Component.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EpicSynth1Component)

void UModularSynthComponent::NoteOn(const float Note, const int32 Velocity, const float Duration)
{
	SynthCommand([this, Note, Velocity, Duration]()
	{
		EpicSynth1.NoteOn((uint32)Note, (int32)Velocity, Duration);
	});
}

void UModularSynthComponent::NoteOff(const float Note, const bool bAllNotesOff, const bool bKillAllNotes)
{
	SynthCommand([this, Note, bAllNotesOff, bKillAllNotes]()
	{
		EpicSynth1.NoteOff((int32)Note, bAllNotesOff, bKillAllNotes);
	});
}

void UModularSynthComponent::SetEnablePolyphony(bool bEnablePolyphony)
{
	SynthCommand([this, bEnablePolyphony]()
	{
		EpicSynth1.SetMonoMode(!bEnablePolyphony);
	});
}

void UModularSynthComponent::SetOscGain(int32 OscIndex, float OscGain)
{
	SynthCommand([this, OscIndex, OscGain]()
	{
		EpicSynth1.SetOscGain(OscIndex, OscGain);
	});
}

void UModularSynthComponent::SetOscGainMod(int32 OscIndex, float OscGainMod)
{
	SynthCommand([this, OscIndex, OscGainMod]()
	{
		EpicSynth1.SetOscGainMod(OscIndex, OscGainMod);
	});
}

void UModularSynthComponent::SetOscFrequencyMod(int32 OscIndex, float OscFreqMod)
{
	SynthCommand([this, OscIndex, OscFreqMod]()
	{
		EpicSynth1.SetOscDetune(OscIndex, OscFreqMod);
	});
}

void UModularSynthComponent::SetOscType(int32 OscIndex, ESynth1OscType OscType)
{
	SynthCommand([this, OscIndex, OscType]()
	{
		EpicSynth1.SetOscType(OscIndex, (Audio::EOsc::Type)OscType);
	});
}

void UModularSynthComponent::SetOscOctave(int32 OscIndex, float Octave)
{
	SynthCommand([this, OscIndex, Octave]()
	{
		EpicSynth1.SetOscOctave(OscIndex, Octave);
	});
}

void UModularSynthComponent::SetOscSemitones(int32 OscIndex, float Semitones)
{
	SynthCommand([this, OscIndex, Semitones]()
	{
		EpicSynth1.SetOscSemitones(OscIndex, Semitones);
	});
}

void UModularSynthComponent::SetOscCents(int32 OscIndex, float Cents)
{
	SynthCommand([this, OscIndex, Cents]()
	{
		EpicSynth1.SetOscCents(OscIndex, Cents);
	});
}

void UModularSynthComponent::SetPitchBend(float PitchBend)
{
	SynthCommand([this, PitchBend]()
	{
		EpicSynth1.SetOscPitchBend(0, PitchBend);
		EpicSynth1.SetOscPitchBend(1, PitchBend);
	});
}

void UModularSynthComponent::SetPortamento(float Portamento)
{
	SynthCommand([this, Portamento]()
	{
		EpicSynth1.SetOscPortamento(Portamento);
	});
}

void UModularSynthComponent::SetOscPulsewidth(int32 OscIndex, float Pulsewidth)
{
	SynthCommand([this, OscIndex, Pulsewidth]()
	{
		EpicSynth1.SetOscPulseWidth(OscIndex, Pulsewidth);
	});
}

void UModularSynthComponent::SetEnableUnison(bool EnableUnison)
{
	SynthCommand([this, EnableUnison]()
	{
		EpicSynth1.SetOscUnison(EnableUnison);
	});
}

void UModularSynthComponent::SetOscSync(const bool bIsSynced)
{
	SynthCommand([this, bIsSynced]()
	{
		EpicSynth1.SetOscSync(bIsSynced);
	});
}

void UModularSynthComponent::SetPan(float Pan)
{
	SynthCommand([this, Pan]()
	{
		EpicSynth1.SetPan(Pan);
	});
}

void UModularSynthComponent::SetSpread(float Spread)
{
	SynthCommand([this, Spread]()
	{
		EpicSynth1.SetOscSpread(Spread);
	});
}

void UModularSynthComponent::SetLFOFrequency(int32 LFOIndex, float FrequencyHz) 
{
	SynthCommand([this, LFOIndex, FrequencyHz]()
	{
		EpicSynth1.SetLFOFrequency(LFOIndex, FrequencyHz);
	});
}

void UModularSynthComponent::SetLFOFrequencyMod(int32 LFOIndex, float FrequencyModHz)
{
	SynthCommand([this, LFOIndex, FrequencyModHz]()
	{
		EpicSynth1.SetLFOFrequencyMod(LFOIndex, FrequencyModHz);
	});
}

void UModularSynthComponent::SetLFOGain(int32 LFOIndex, float Gain) 
{
	SynthCommand([this, LFOIndex, Gain]()
	{
		EpicSynth1.SetLFOGain(LFOIndex, Gain);
	});
}

void UModularSynthComponent::SetLFOGainMod(int32 LFOIndex, float GainMod)
{
	SynthCommand([this, LFOIndex, GainMod]()
	{
		EpicSynth1.SetLFOGainMod(LFOIndex, GainMod);
	});
}

void UModularSynthComponent::SetLFOType(int32 LFOIndex, ESynthLFOType LFOType)
{
	SynthCommand([this, LFOIndex, LFOType]()
	{
		EpicSynth1.SetLFOType(LFOIndex, (Audio::ELFO::Type)LFOType);
	});
}

void UModularSynthComponent::SetLFOMode(int32 LFOIndex, ESynthLFOMode LFOMode)
{
	SynthCommand([this, LFOIndex, LFOMode]()
	{
		EpicSynth1.SetLFOMode(LFOIndex, (Audio::ELFOMode::Type)LFOMode);
	});
}

void UModularSynthComponent::SetLFOPatch(int32 LFOIndex, ESynthLFOPatchType LFOPatchType)
{
	SynthCommand([this, LFOIndex, LFOPatchType]()
	{
		EpicSynth1.SetLFOPatch(LFOIndex, LFOPatchType);
	});
}

void UModularSynthComponent::SetGainDb(float GainDb)
{
	SynthCommand([this, GainDb]()
	{
		EpicSynth1.SetGainDb(GainDb);
	});
}

void UModularSynthComponent::SetAttackTime(float AttackTimeMsec)
{
	SynthCommand([this, AttackTimeMsec]()
	{
		EpicSynth1.SetEnvAttackTime(AttackTimeMsec);
	});
}

void UModularSynthComponent::SetDecayTime(float DecayTimeMsec)
{
	SynthCommand([this, DecayTimeMsec]()
	{
		EpicSynth1.SetEnvDecayTime(DecayTimeMsec);
	});
}

void UModularSynthComponent::SetSustainGain(float SustainGain)
{
	SynthCommand([this, SustainGain]()
	{
		EpicSynth1.SetEnvSustainGain(SustainGain);
	});
}

void UModularSynthComponent::SetReleaseTime(float ReleaseTimeMsec)
{
	SynthCommand([this, ReleaseTimeMsec]()
	{
		EpicSynth1.SetEnvReleaseTime(ReleaseTimeMsec);
	});
}

void UModularSynthComponent::SetModEnvPatch(const ESynthModEnvPatch InPatchType)
{
	SynthCommand([this, InPatchType]()
	{
		EpicSynth1.SetModEnvPatch(InPatchType);
	});
}

void UModularSynthComponent::SetModEnvBiasPatch(const ESynthModEnvBiasPatch InPatchType)
{
	SynthCommand([this, InPatchType]()
	{
		EpicSynth1.SetModEnvBiasPatch(InPatchType);
	});
}

void UModularSynthComponent::SetModEnvInvert(const bool bInInvert)
{
	SynthCommand([this, bInInvert]()
	{
		EpicSynth1.SetModEnvInvert(bInInvert);
	});
}

void UModularSynthComponent::SetModEnvBiasInvert(const bool bInInvert)
{
	SynthCommand([this, bInInvert]()
	{
		EpicSynth1.SetModEnvBiasInvert(bInInvert);
	});
}

void UModularSynthComponent::SetModEnvDepth(const float Depth)
{
	SynthCommand([this, Depth]()
	{
		EpicSynth1.SetModEnvDepth(Depth);
	});
}

void UModularSynthComponent::SetModEnvAttackTime(const float AttackTimeMsec)
{
	SynthCommand([this, AttackTimeMsec]()
	{
		EpicSynth1.SetModEnvAttackTime(AttackTimeMsec);
	});
}

void UModularSynthComponent::SetModEnvDecayTime(const float DecayTimeMsec)
{
	SynthCommand([this, DecayTimeMsec]()
	{
		EpicSynth1.SetModEnvDecayTime(DecayTimeMsec);
	});
}

void UModularSynthComponent::SetModEnvSustainGain(const float SustainGain)
{
	SynthCommand([this, SustainGain]()
	{
		EpicSynth1.SetModEnvSustainGain(SustainGain);
	});
}

void UModularSynthComponent::SetModEnvReleaseTime(const float Release)
{
	SynthCommand([this, Release]()
	{
		EpicSynth1.SetModEnvReleaseTime(Release);
	});
}

void UModularSynthComponent::SetEnableLegato(bool LegatoEnabled)
{
	SynthCommand([this, LegatoEnabled]()
	{
		EpicSynth1.SetEnvLegatoEnabled(LegatoEnabled);
	});
}

void UModularSynthComponent::SetEnableRetrigger(bool RetriggerEnabled)
{
	SynthCommand([this, RetriggerEnabled]()
	{
		EpicSynth1.SetEnvRetriggerMode(RetriggerEnabled);
	});
}

void UModularSynthComponent::SetFilterFrequency(float FilterFrequencyHz)
{
	SynthCommand([this, FilterFrequencyHz]()
	{
		EpicSynth1.SetFilterFrequency(FilterFrequencyHz);
	});
}

void UModularSynthComponent::SetFilterFrequencyMod(float FilterFrequencyHz)
{
	SynthCommand([this, FilterFrequencyHz]()
	{
		EpicSynth1.SetFilterFrequencyMod(FilterFrequencyHz);
	});
}

void UModularSynthComponent::SetFilterQ(float FilterQ)
{
	SynthCommand([this, FilterQ]()
	{
		EpicSynth1.SetFilterQ(FilterQ);
	});
}

void UModularSynthComponent::SetFilterQMod(float FilterQMod)
{
	SynthCommand([this, FilterQMod]()
	{
		EpicSynth1.SetFilterQMod(FilterQMod);
	});
}

void UModularSynthComponent::SetFilterType(ESynthFilterType FilterType)
{
	SynthCommand([this, FilterType]()
	{
		EpicSynth1.SetFilterType((Audio::EFilter::Type)FilterType);
	});
}

void UModularSynthComponent::SetFilterAlgorithm(ESynthFilterAlgorithm FilterAlgorithm)
{
	SynthCommand([this, FilterAlgorithm]()
	{
		EpicSynth1.SetFilterAlgorithm(FilterAlgorithm);
	});
}

void UModularSynthComponent::SetStereoDelayIsEnabled(bool StereoDelayEnabled)
{
	SynthCommand([this, StereoDelayEnabled]()
	{
		EpicSynth1.SetStereoDelayIsEnabled(StereoDelayEnabled);
	});
}

void UModularSynthComponent::SetStereoDelayMode(ESynthStereoDelayMode StereoDelayMode)
{
	SynthCommand([this, StereoDelayMode]()
	{
		EpicSynth1.SetStereoDelayMode((Audio::EStereoDelayMode::Type)StereoDelayMode);
	});
}

void UModularSynthComponent::SetStereoDelayTime(float DelayTimeMsec)
{
	SynthCommand([this, DelayTimeMsec]()
	{
		EpicSynth1.SetStereoDelayTimeMsec(DelayTimeMsec);
	});
}

void UModularSynthComponent::SetStereoDelayFeedback(float DelayFeedback)
{
	SynthCommand([this, DelayFeedback]()
	{
		EpicSynth1.SetStereoDelayFeedback(DelayFeedback);
	});
}

void UModularSynthComponent::SetStereoDelayWetlevel(float DelayWetlevel)
{
	SynthCommand([this, DelayWetlevel]()
	{
		EpicSynth1.SetStereoDelayWetLevel(DelayWetlevel);
	});
}

void UModularSynthComponent::SetStereoDelayRatio(float DelayRatio)
{
	SynthCommand([this, DelayRatio]()
	{
		EpicSynth1.SetStereoDelayRatio(DelayRatio);
	});
}

void UModularSynthComponent::SetChorusEnabled(bool EnableChorus)
{
	SynthCommand([this, EnableChorus]()
	{
		EpicSynth1.SetChorusEnabled(EnableChorus);
	});
}

void UModularSynthComponent::SetChorusDepth(float Depth)
{
	SynthCommand([this, Depth]()
	{
		EpicSynth1.SetChorusDepth(Audio::EChorusDelays::Left, Depth);
		EpicSynth1.SetChorusDepth(Audio::EChorusDelays::Center, Depth);
		EpicSynth1.SetChorusDepth(Audio::EChorusDelays::Right, Depth);
	});
}

void UModularSynthComponent::SetChorusFeedback(float Feedback)
{
	SynthCommand([this, Feedback]()
	{
		EpicSynth1.SetChorusFeedback(Audio::EChorusDelays::Left, Feedback);
		EpicSynth1.SetChorusFeedback(Audio::EChorusDelays::Center, Feedback);
		EpicSynth1.SetChorusFeedback(Audio::EChorusDelays::Right, Feedback);
	});
}

void UModularSynthComponent::SetChorusFrequency(float Frequency) 
{
	SynthCommand([this, Frequency]()
	{
		EpicSynth1.SetChorusFrequency(Audio::EChorusDelays::Left, Frequency);
		EpicSynth1.SetChorusFrequency(Audio::EChorusDelays::Center, Frequency);
		EpicSynth1.SetChorusFrequency(Audio::EChorusDelays::Right, Frequency);
	});
}

void UModularSynthComponent::SetSynthPreset(const FModularSynthPreset& SynthPreset)
{
	SynthCommand([this, SynthPreset]()
	{
		EpicSynth1.SetMonoMode(!SynthPreset.bEnablePolyphony);

		EpicSynth1.SetOscType(0, (Audio::EOsc::Type)SynthPreset.Osc1Type);
		EpicSynth1.SetOscGain(0, SynthPreset.Osc1Gain);
		EpicSynth1.SetOscOctave(0, SynthPreset.Osc1Octave);
		EpicSynth1.SetOscSemitones(0, SynthPreset.Osc1Semitones);
		EpicSynth1.SetOscCents(0, SynthPreset.Osc1Cents);
		EpicSynth1.SetOscPulseWidth(0, SynthPreset.Osc1PulseWidth);

		EpicSynth1.SetOscType(1, (Audio::EOsc::Type)SynthPreset.Osc2Type);
		EpicSynth1.SetOscGain(1, SynthPreset.Osc2Gain);
		EpicSynth1.SetOscOctave(1, SynthPreset.Osc2Octave);
		EpicSynth1.SetOscSemitones(1, SynthPreset.Osc2Semitones);
		EpicSynth1.SetOscCents(1, SynthPreset.Osc2Cents);
		EpicSynth1.SetOscPulseWidth(1, SynthPreset.Osc2PulseWidth);

		EpicSynth1.SetOscPortamento(SynthPreset.Portamento);
		EpicSynth1.SetOscUnison(SynthPreset.bEnableUnison);
		EpicSynth1.SetOscSync(SynthPreset.bEnableOscillatorSync);

		EpicSynth1.SetOscSpread(SynthPreset.Spread);
		EpicSynth1.SetPan(SynthPreset.Pan);

		EpicSynth1.SetLFOFrequency(0, SynthPreset.LFO1Frequency);
		EpicSynth1.SetLFOGain(0, SynthPreset.LFO1Gain);
		EpicSynth1.SetLFOType(0, (Audio::ELFO::Type)SynthPreset.LFO1Type);
		EpicSynth1.SetLFOMode(0, (Audio::ELFOMode::Type)SynthPreset.LFO1Mode);
		EpicSynth1.SetLFOPatch(0, SynthPreset.LFO1PatchType);

		EpicSynth1.SetLFOFrequency(1, SynthPreset.LFO2Frequency);
		EpicSynth1.SetLFOGain(1, SynthPreset.LFO2Gain);
		EpicSynth1.SetLFOType(1, (Audio::ELFO::Type)SynthPreset.LFO2Type);
		EpicSynth1.SetLFOMode(1, (Audio::ELFOMode::Type)SynthPreset.LFO2Mode);
		EpicSynth1.SetLFOPatch(1, SynthPreset.LFO2PatchType);

		EpicSynth1.SetGainDb(SynthPreset.GainDb);
		
		EpicSynth1.SetEnvAttackTime(SynthPreset.AttackTime);
		EpicSynth1.SetEnvDecayTime(SynthPreset.DecayTime);
		EpicSynth1.SetEnvSustainGain(SynthPreset.SustainGain);
		EpicSynth1.SetEnvReleaseTime(SynthPreset.ReleaseTime);

		EpicSynth1.SetModEnvPatch(SynthPreset.ModEnvPatchType);
		EpicSynth1.SetModEnvBiasPatch(SynthPreset.ModEnvBiasPatchType);
		EpicSynth1.SetModEnvInvert(SynthPreset.bInvertModulationEnvelope);
		EpicSynth1.SetModEnvBiasInvert(SynthPreset.bInvertModulationEnvelopeBias);
		EpicSynth1.SetModEnvDepth(SynthPreset.ModulationEnvelopeDepth);
		EpicSynth1.SetModEnvAttackTime(SynthPreset.ModulationEnvelopeAttackTime);
		EpicSynth1.SetModEnvDecayTime(SynthPreset.ModulationEnvelopeDecayTime);
		EpicSynth1.SetModEnvSustainGain(SynthPreset.ModulationEnvelopeSustainGain);
		EpicSynth1.SetModEnvReleaseTime(SynthPreset.ModulationEnvelopeReleaseTime);

		EpicSynth1.SetEnvLegatoEnabled(SynthPreset.bLegato);
		EpicSynth1.SetEnvRetriggerMode(SynthPreset.bRetrigger);

		EpicSynth1.SetFilterFrequency(SynthPreset.FilterFrequency);
		EpicSynth1.SetFilterQ(SynthPreset.FilterQ);
		EpicSynth1.SetFilterType((Audio::EFilter::Type)SynthPreset.FilterType);
		EpicSynth1.SetFilterAlgorithm(SynthPreset.FilterAlgorithm);

		EpicSynth1.SetStereoDelayIsEnabled(SynthPreset.bStereoDelayEnabled);
		EpicSynth1.SetStereoDelayMode((Audio::EStereoDelayMode::Type)SynthPreset.StereoDelayMode);
		EpicSynth1.SetStereoDelayTimeMsec(SynthPreset.StereoDelayTime);
		EpicSynth1.SetStereoDelayFeedback(SynthPreset.StereoDelayFeedback);
		EpicSynth1.SetStereoDelayWetLevel(SynthPreset.StereoDelayWetlevel);
		EpicSynth1.SetStereoDelayRatio(SynthPreset.StereoDelayRatio);

		EpicSynth1.SetChorusEnabled(SynthPreset.bChorusEnabled);
		EpicSynth1.SetChorusDepth(Audio::EChorusDelays::Left, SynthPreset.ChorusDepth);
		EpicSynth1.SetChorusDepth(Audio::EChorusDelays::Center, SynthPreset.ChorusDepth);
		EpicSynth1.SetChorusDepth(Audio::EChorusDelays::Right, SynthPreset.ChorusDepth);
		EpicSynth1.SetChorusFeedback(Audio::EChorusDelays::Left, SynthPreset.ChorusFeedback);
		EpicSynth1.SetChorusFeedback(Audio::EChorusDelays::Center, SynthPreset.ChorusFeedback);
		EpicSynth1.SetChorusFeedback(Audio::EChorusDelays::Right, SynthPreset.ChorusFeedback);
		EpicSynth1.SetChorusFrequency(Audio::EChorusDelays::Left, SynthPreset.ChorusFrequency);
		EpicSynth1.SetChorusFrequency(Audio::EChorusDelays::Center, SynthPreset.ChorusFrequency);
		EpicSynth1.SetChorusFrequency(Audio::EChorusDelays::Right, SynthPreset.ChorusFrequency);

		// Make sure we clear out any existing patches
		EpicSynth1.ClearPatches();

		// Now loop through all the patches
		for (const FEpicSynth1Patch& Patch : SynthPreset.Patches)
		{
			if (Patch.PatchCables.Num() > 0)
			{
				EpicSynth1.CreatePatch(Patch.PatchSource, Patch.PatchCables, true);
			}
		}

	});
}

FPatchId UModularSynthComponent::CreatePatch(const ESynth1PatchSource PatchSource, const TArray<FSynth1PatchCable>& PatchCables, const bool bEnabledByDefault)
{
	return EpicSynth1.CreatePatch(PatchSource, PatchCables, bEnabledByDefault);
}

bool UModularSynthComponent::SetEnablePatch(const FPatchId PatchId, const bool bIsEnabled)
{
	return EpicSynth1.SetEnablePatch(PatchId, bIsEnabled);
}

UModularSynthComponent::UModularSynthComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	VoiceCount = 8;
}

bool UModularSynthComponent::Init(int32& SampleRate)
{
	NumChannels = 2;

	EpicSynth1.Init(SampleRate, VoiceCount);

	// Set default parameters
	SetEnablePolyphony(false);
	SetOscType(0, ESynth1OscType::Saw);
	SetOscType(1, ESynth1OscType::Saw);
	SetOscCents(1, 2.5f);
	SetOscPulsewidth(0, 0.5f);
	SetOscPulsewidth(1, 0.5f);
	SetEnableUnison(false);
	SetSpread(0.5f);
	SetGainDb(-3.0f);
	SetAttackTime(10.0f);
	SetDecayTime(100.0f);
	SetSustainGain(0.707f);
	SetReleaseTime(5000.0f);
	SetEnableLegato(true);
	SetEnableRetrigger(false);
	SetFilterFrequency(1200.0f);
	SetFilterQ(2.0f);
	SetFilterAlgorithm(ESynthFilterAlgorithm::Ladder);
	SetStereoDelayIsEnabled(true);
	SetStereoDelayMode(ESynthStereoDelayMode::PingPong);
	SetStereoDelayRatio(0.2f);
	SetStereoDelayFeedback(0.7f);
	SetStereoDelayWetlevel(0.3f);
	SetChorusEnabled(false);

	return true;
}

int32 UModularSynthComponent::OnGenerateAudio(float* OutAudio, int32 NumSamples)
{
	for (int32 SampleIndex = 0; SampleIndex < NumSamples; SampleIndex += NumChannels)
	{
		EpicSynth1.GenerateFrame(&OutAudio[SampleIndex]);
	}
	return NumSamples;
}


void UModularSynthLibrary::AddModularSynthPresetToBankAsset(UModularSynthPresetBank* InBank, const FModularSynthPreset& Preset, const FString& PresetName)
{
	if (GIsEditor)
	{
		FModularSynthPresetBankEntry NewBankEntry;
		NewBankEntry.PresetName = PresetName;
		NewBankEntry.Preset = Preset;

		InBank->Presets.Add(NewBankEntry);
		InBank->MarkPackageDirty();
	}
}

