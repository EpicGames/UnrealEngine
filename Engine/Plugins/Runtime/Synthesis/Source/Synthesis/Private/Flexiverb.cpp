// Copyright Epic Games, Inc. All Rights Reserved.

#include "Flexiverb.h"
#include "DSP/AllPassFilter.h"
#include "DSP/OnePole.h"

namespace Audio
{
	// Delay line lengths used, in samples. These are chosen because they are mutually prime numbers, extended by a
	// constant to minimize ringing products while avoiding any delay line path that would cause noticeable echo.
	static const float DelayLineSampleLengths[] = { 16.0f * 142.0f, 16.0f * 107.0f, 16.0f * 379.0f, 16.0f * 277.0f, 16.0f * 279.0f,
		                                            16.0f * 137.0f, 16.0f * 213.0f, 16.0f * 327.0f, 16.0f * 2001.0f, 16.0f * 987.0f, 
		                                            16.0f * 826.0f };

	// APF Lengths, in samples. These were in part chosen based on APF lengths described here:
	// https://ccrma.stanford.edu/~jos/pasp/Freeverb.html
	static const float APFAmountFactor[] = { 225.0f, 55.0f, 441.0f, 341.0f, 327.0f, 497.0f, 551.0f, 768.0f, 1013.0f, 1139.0f, 1470.0f,
		                                     1532.0f, 1709.0f, 1941.0f, 2037.0f };
	
	// The amount, in Hz, to randomize the cutoff of each individual dampening filter:
	static const float DiffusenessFrequencyFactor = 150.0f;

	// Number of APF filters, in series, to process the input through before sending it through the FDN.
	static const int32 NumberOfAPFDiffusers = 4;

	FFlexiverb::FFlexiverb()
		: DecayFactor(0.0f)
		, ScatteringMatrixLength(0)
	{
	}

	FFlexiverb::~FFlexiverb()
	{
	}

	void FFlexiverb::Init(const int32 InSampleRate, const FFlexiverbSettings Settings)
	{
		SampleRate = InSampleRate;
		PreDelayLine.Init(SampleRate, 2.0f);

		SetSettings(Settings);

		APFArray.Init(FDelayAPF(), NumberOfAPFDiffusers);
		int32 Index = 0;
		for (FDelayAPF& Filter : APFArray)
		{
			Filter.Init(SampleRate);
			float DelayTime = APFAmountFactor[Index++];
			Filter.SetDelaySamples(DelayTime);
		}
	}

	void FFlexiverb::SetSettings(const FFlexiverbSettings& InSettings)
	{
		if (InSettings.Complexity != ScatteringMatrixLength)
		{
			UpdateComplexity(InSettings.Complexity);
		}

		PrivateSettings = InSettings;
		Update();
	}
	
	void FFlexiverb::Update()
	{
		// Calculate the decay factor based on a logarithmic scale:
		DecayFactor = 0.49999f + (FMath::LogX(60.0f, PrivateSettings.DecayTime / 12.0f) + 1.0f) * 0.5f;

		// Ensure that our decay factor still ensures that our FDN system is stable:
		check(DecayFactor < 1.0f);

		//Set our pre-delay line:
		PreDelayLine.SetDelayMsec(PrivateSettings.PreDelay);

		for (int32 Column = 0; Column < ScatteringMatrixLength; Column++)
		{
			// Determine a transition frequency for this dampening filter based on the given room centroid:
			float TransitionFrequencyHz = (FMath::FRand() - 0.5f) * DiffusenessFrequencyFactor + PrivateSettings.RoomDampening;

			// Set the one pole filter to the appropriate normalized frequency:
			DampeningArray[Column].SetFrequency(0.5f * TransitionFrequencyHz / SampleRate);
		}
	}

	void FFlexiverb::UpdateComplexity(const int32 Complexity)
	{
		ScatteringMatrixLength = Complexity * 2 + 1;
		DelayLines.Init(FDelayAPF(), ScatteringMatrixLength);
		int32 Index = 0;
		for (FDelayAPF& Delay : DelayLines)
		{
			Delay.Init(SampleRate, 10.0f);
			Delay.SetDelaySamples(DelayLineSampleLengths[Index++]);
		}

		DampeningArray.Init(FOnePoleLPF(), ScatteringMatrixLength);

		
	}

	void FFlexiverb::ProcessAudioFrame(const float* InBuffer, const int32 InChannels, float* OutBuffer, const int32 OutChannels)
	{
		float DiffusedInput = 0.0f;
		float DryInput = 0.0f;
		float Temp;
		float OutputAccumulator = 0.0f;

		// Sum the input channels into a mono sample
		for (int32 Channel = 0; Channel < InChannels; Channel++)
		{
			DryInput += InBuffer[Channel];
		}

		DryInput /= InChannels;

		// Process the input through our pre-delay:
		DryInput = PreDelayLine.ProcessAudioSample(DryInput);

		// Process the pre-delay output through a series of APFs:
		FDelayAPF* APFArrayPtr = APFArray.GetData();
		for (int32 APFIndex = 0; APFIndex < APFArray.Num(); ++APFIndex)
		{
			DryInput = APFArrayPtr[APFIndex].ProcessAudioSample(DryInput);
		}
		
		// Process the APF series output through our FDN.
		FDelayAPF* DelayLinesPtr = DelayLines.GetData();
		FOnePoleLPF* DampeningArrayPtr = DampeningArray.GetData();
		for (int32 DelayColumn = 0; DelayColumn < ScatteringMatrixLength; DelayColumn++)
		{
			Temp = DryInput;

			// Tap and sum our other delay lines into the current delay line:
			for (int32 DelayRow = 0; DelayRow < ScatteringMatrixLength; ++DelayRow)
			{
				const int32 NextIndex = (DelayColumn + 1) % ScatteringMatrixLength;
				Temp += DelayLinesPtr[NextIndex].Read();

				// Negate all scattering matrix values not on the diagonal, according to
				// the Householder feedback matrix:
				if (DelayRow != DelayColumn)
				{
					Temp *= -1.0f;
				}
			}

			// Apply our decay factor:
			Temp *= DecayFactor;
			
			// Process our delay line input through it's corresponding LPF:
			Temp = DampeningArrayPtr[DelayColumn].ProcessAudioSample(Temp);

			// Finally, process the delay line.
			Temp = DelayLinesPtr[DelayColumn].ProcessAudioSample(Temp);

			// Tap out of each delay line and sum into our OutputAccumulator:
			OutputAccumulator += Temp;
		}

		// Downmix our reverb to the number of channels requested for output.
		OutputAccumulator /= OutChannels;
		for (int32 Channel = 0; Channel < OutChannels; Channel++)
		{
			OutBuffer[Channel] = OutputAccumulator;
		}
	}
}
