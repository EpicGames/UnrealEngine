// Copyright Epic Games, Inc. All Rights Reserved.


#include "MonoWaveTable.h"

namespace Audio
{
	// keep from popping *too* hard on short attacks
	static const float MinimumEnvelopeTimeMSEC  = 4;

	FMonoWaveTable::FMonoWaveTable() { }

	FMonoWaveTable::~FMonoWaveTable() { }

	void FMonoWaveTable::Init(int32& InSampleRate, DefaultWaveTableIndexType InNumTableElements, float InWaveTableResolution)
	{
		// Note: purposefully not reseting envelopes etc.
		// this is called when table elements are added/removed
		// so synth can keep rendering through init
		OutputSampleRate = static_cast<float>(InSampleRate);
		WaveTableResolution = InWaveTableResolution;

		// Reset tables
		WaveTableMatrix.Init({}, InNumTableElements);

		if (!bInitialized)
		{
			// update members
			WaveTablePosition = 0.5f; // middle of tables
			LastWaveTablePosition = WaveTablePosition;
			FrequencyInHz = 220.0f;
			Phase = 0.0f;

			// LFO
			PosLfo.Init(OutputSampleRate);
			PosLfo.SetBipolar(false);
			PosLfo.SetType(Audio::ELFO::Sine);
			PosLfo.SetFrequency(0.0f);
			PosLfo.SetGainMod(0.0f);
			PosLfo.Start();

			// Envelopes
			AmpEnv.SetAttackTime(1);
			AmpEnv.SetSustainGain(1.0f);
			AmpEnv.SetReleaseTime(4000);
			AmpEnv.SetDepth(1.0f);	
			AmpEnv.SetLegato(true);
			AmpEnv.SetRetrigger(true);

			LPFEnv.SetAttackTime(1);
			LPFEnv.SetSustainGain(1.0f);
			LPFEnv.SetReleaseTime(4000);
			LPFEnv.SetDepth(1.0f);
			LPFEnv.SetLegato(true);
			LPFEnv.SetRetrigger(true);

			// Filters
			LPFBaseFreqHz = 1000;
			LPFMaxModOffsetOctaves = 9;
			LPFCeilingHz = 8000;
			LastLPFFreqHz = LPFBaseFreqHz;
			LPF.Init(OutputSampleRate, 1);
			LPF.SetFrequency(LPFBaseFreqHz);
			LPF.SetQ(3.0f);

			// Osc
			PitchOffsetCents = 0.0f;

			bInitialized = true;
		}
	}

	void FMonoWaveTable::UpdateWaveTable(DefaultWaveTableIndexType Index, const TArray<float>& InTable)
	{
		check(Index <= WaveTableMatrix.Num());
			
		if (Index == WaveTableMatrix.Num())
		{
			TSampleBuffer<float> TempBuffer(WaveTableMatrix[Index - 1]);
			WaveTableMatrix.Add(TempBuffer);
		}

		WaveTableMatrix[Index].CopyFrom(InTable, 1, OutputSampleRate);
	}

	void FMonoWaveTable::NoteOn(float InFrequency, float Velocity)
	{
		if (Velocity < SMALL_NUMBER)
		{
			NoteOff(InFrequency);
		}
		else
		{
			FrequencyInHz = InFrequency;

			NoteStack.Push(InFrequency);
			
			AmpEnv.StartLegato(Velocity);
			LPFEnv.StartLegato();

			bIsPlaying = true;
		}
	}

	void FMonoWaveTable::NoteOff(float InFrequency)
	{
		NoteStack.Remove(InFrequency);

		if (!NoteStack.Num() && !bIsSustainPedalPressed)
		{
			AmpEnv.Stop();
			LPFEnv.Stop();
		}
		else if(NoteStack.Num())
		{
			FrequencyInHz = NoteStack.Top();
		}
	}

	DefaultWaveTableIndexType FMonoWaveTable::OnGenerateAudio(float* OutAudio, DefaultWaveTableIndexType NumSamples)
	{
		// check for valid table data
		if (!WaveTableMatrix.Num())
		{
			return 0;
		}

		const int32 NumTables = WaveTableMatrix.Num();
		for (int32 i = 0; i < NumTables; ++i)
		{
			if (WaveTableMatrix[i].GetNumFrames() == 0)
			{
				return 0;
			}
		}

		// Calculate sample increment based on sample rate and 
		// num samples it should take to get through WT
		const float TargetFrequency = (FrequencyInHz * Audio::GetFrequencyMultiplier(PitchOffsetCents / 100.0f));
		check(TargetFrequency >= SMALL_NUMBER);

		const float TargetPeriod = OutputSampleRate / TargetFrequency;
		check(TargetPeriod >= SMALL_NUMBER); 

		const float TableStepSize = WaveTableResolution / TargetPeriod; // Sample increment

		// Update filter coefficients outside of loop (cheaper. let zipper for now)
		// current cutoff = base cutoff * 2^(envelopeValue * Max number of octaves)
		// (written this way for debugging clarity for now)
		float EnvOut = LPFEnv.Generate();
		float Exponent = EnvOut * LPFMaxModOffsetOctaves;
		float PitchScalar = FMath::Pow(2.0f, Exponent);
		const float NewLPFFreq = FMath::Clamp(LPFBaseFreqHz * PitchScalar, 0.0f, LPFCeilingHz);

		// Calculate interpolation step sizes
		const float LPFStepSize = (NewLPFFreq - LastLPFFreqHz) / static_cast<float>(NumSamples);
		const float AmpSustainStepSize = (NewAmpSustain - LastAmpSustain) / static_cast<float>(NumSamples);
		const float PosStep = (WaveTablePosition - LastWaveTablePosition) / static_cast<float>(NumSamples);

		// perform linear interpolation and output
		for (DefaultWaveTableIndexType i = 0; i < NumSamples; ++i)
		{

			float WaveTableOutput, LPFOutput;

			// update Filter
			// doing the expensive way (every frame) for now for short A/D envelope-driven LPF cutoff
			LPF.SetFrequency(LastLPFFreqHz += LPFStepSize);
			LPF.Update();

			// tick even if not used per-frame
			LPFEnv.Generate();
			PosLfo.Update();

			// get WT sample
			const float CurrentModulatedPosition = (LastWaveTablePosition += PosStep) + PosLfo.Generate();

			// updates WaveTableOutputLookupOutput array (returns wrapped Phase)
			Phase = UpdateWaveTableLookupOutput(Phase);
			WaveTableOutput = GetMonoSynthOutput(CurrentModulatedPosition);

			// apply LPF
			LPF.ProcessAudioFrame(&WaveTableOutput, &LPFOutput);

			// apply ENV
			AmpEnv.SetSustainGain(LastAmpSustain += AmpSustainStepSize);
			OutAudio[i] = AmpEnv.Generate() * LPFOutput;

			Phase += TableStepSize;
		}

		// see if the amp has finished fading out this callback
		bIsPlaying = !AmpEnv.IsDone();

		// update "current" values for interpolation
		LastWaveTablePosition = WaveTablePosition;
		LastLPFFreqHz = NewLPFFreq;
		LastAmpSustain = NewAmpSustain;

		return NumSamples;
	}

	float FMonoWaveTable::UpdateWaveTableLookupOutput(float InPhaseIndex)
	{
		const int32 NumTables = WaveTableMatrix.Num();
		WaveTableLookupOutput.SetNumUninitialized(NumTables);

		TArray<float> Temp;

		for (int32 i = 0; i < NumTables; ++i)
		{
 			InPhaseIndex = WaveTableMatrix[i].GetAudioFrameAtFractionalIndex(InPhaseIndex, Temp);
 			WaveTableLookupOutput[i] = Temp[0];
		}

		return InPhaseIndex;
	}

	float FMonoWaveTable::GetMonoSynthOutput(float InTablePosition)
	{
		InTablePosition = FMath::Clamp(InTablePosition, 0.0f, 1.0f);

		// scale modifier to number of tables
		InTablePosition *= static_cast<float>(WaveTableMatrix.Num() - 1);

		// prep interpolation
		int32 WholeIndexA = FMath::FloorToInt(InTablePosition);
		int32 WholeIndexB = WholeIndexA + 1;

		if (WholeIndexB == WaveTableMatrix.Num())
		{
			// interpolating between last and first table
			WholeIndexB = 0;
		}

		const float Alpha = FMath::Fmod(InTablePosition, 1.0f);
		const float SampleA = WaveTableLookupOutput[WholeIndexA];
		const float SampleB = WaveTableLookupOutput[WholeIndexB];

		return FMath::Lerp(SampleA, SampleB, Alpha);
	}

	void FMonoWaveTable::SetPosLFOFrequency(const float InLFOFrequency)	
	{
		PosLfo.SetFrequency(InLFOFrequency);
	}

	void FMonoWaveTable::SetPosLFODepth(const float InLFODepth)
	{
		PosLfo.SetGainMod(InLFODepth);
	}

	void FMonoWaveTable::SetPosLFOType(const ELFO::Type InLFOType)
	{
		PosLfo.SetType(InLFOType);
	}

	void FMonoWaveTable::SetAmpEnvelopeAttackTime(const float InAttackTimeMsec)
	{
		AmpEnv.SetAttackTime(FMath::Max(InAttackTimeMsec, MinimumEnvelopeTimeMSEC));
	}

	void FMonoWaveTable::SetAmpEnvelopeDecayTime(const float InDecayTimeMsec)
	{
		AmpEnv.SetDecayTime(FMath::Max(InDecayTimeMsec, MinimumEnvelopeTimeMSEC));
	}
	
	void FMonoWaveTable::SetAmpEnvelopeSustainGain(const float InSustainGain)
	{
		NewAmpSustain = InSustainGain;
	}
	
	void FMonoWaveTable::SetAmpEnvelopeReleaseTime(const float InReleaseTimeMsec)
	{
		AmpEnv.SetReleaseTime(FMath::Max(InReleaseTimeMsec, MinimumEnvelopeTimeMSEC));
	}
	
	void FMonoWaveTable::SetAmpEnvelopeInvert(const bool bInInvert)
	{
		AmpEnv.SetReleaseTime(bInInvert);
	}
	
	void FMonoWaveTable::SetAmpEnvelopeBiasInvert(const bool bInBiasInvert)
	{
		AmpEnv.SetBiasInvert(bInBiasInvert);
	}
	
	void FMonoWaveTable::SetAmpEnvelopeDepth(const float InDepth)
	{
		AmpEnv.SetDepth(InDepth);
	}
	
	void FMonoWaveTable::SetAmpEnvelopeBiasDepth(const float InDepth)
	{
		AmpEnv.SetBiasDepth(InDepth);
	}
	
	void FMonoWaveTable::SetFilterEnvelopeAttackTime(const float InAttackTimeMsec)
	{
		LPFEnv.SetAttackTime(FMath::Max(InAttackTimeMsec, MinimumEnvelopeTimeMSEC));
	}
	
	void FMonoWaveTable::SetFilterEnvelopenDecayTime(const float InDecayTimeMsec)
	{
		LPFEnv.SetDecayTime(FMath::Max(InDecayTimeMsec, MinimumEnvelopeTimeMSEC));
	}
	
	void FMonoWaveTable::SetFilterEnvelopeSustainGain(const float InSustainGain)
	{
		LPFEnv.SetSustainGain(InSustainGain);
	}
	
	void FMonoWaveTable::SetFilterEnvelopeReleaseTime(const float InReleaseTimeMsec)
	{
		LPFEnv.SetReleaseTime(FMath::Max(InReleaseTimeMsec, MinimumEnvelopeTimeMSEC));
	}
	
	void FMonoWaveTable::SetFilterEnvelopeInvert(const bool bInInvert)
	{
		LPFEnv.SetInvert(bInInvert);
	}
	
	void FMonoWaveTable::SetFilterEnvelopeBiasInvert(const bool bInBiasInvert)
	{
		LPFEnv.SetBiasInvert(bInBiasInvert);
	}
	
	void FMonoWaveTable::SetFilterEnvelopeDepth(const float InDepth)
	{
		LPFEnv.SetDepth(InDepth);
	}
	
	void FMonoWaveTable::SetFilterEnvelopeBiasDepth(const float InDepth)
	{
		LPFEnv.SetBiasDepth(InDepth);
	}
	
	void FMonoWaveTable::SetPositionEnvelopeAttackTime(const float InAttackTimeMsec)
	{
		PosEnv.SetAttackTime(InAttackTimeMsec);
	}
	
	void FMonoWaveTable::SetPositionEnvelopeDecayTime(const float InDecayTimeMsec)
	{
		PosEnv.SetDecayTime(InDecayTimeMsec);
	}
	
	void FMonoWaveTable::SetPositionEnvelopeSustainGain(const float InSustainGain)
	{
		PosEnv.SetSustainGain(InSustainGain);
	}
	
	void FMonoWaveTable::SetPositionEnvelopeReleaseTime(const float InReleaseTimeMsec)
	{
		PosEnv.SetReleaseTime(InReleaseTimeMsec);
	}
	
	void FMonoWaveTable::SetPositionEnvelopeInvert(const bool bInInvert)
	{
		PosEnv.SetInvert(bInInvert);
	}
	
	void FMonoWaveTable::SetPositionEnvelopeBiasInvert(const bool bInBiasInvert)
	{
		PosEnv.SetBiasInvert(bInBiasInvert);
	}
	
	void FMonoWaveTable::SetPositionEnvelopeDepth(const float InDepth)
	{
		PosEnv.SetDepth(InDepth);
	}
	
	void FMonoWaveTable::SetPositionEnvelopeBiasDepth(const float InDepth)
	{
		PosEnv.SetBiasDepth(InDepth);
	}

	void FMonoWaveTable::SetSustainPedalPressed(bool bInSustainPedalState)
	{
		// pedal released
		if (!bInSustainPedalState && !NoteStack.Num())
		{
			AmpEnv.Stop();
			LPFEnv.Stop();
		}

		bIsSustainPedalPressed = bInSustainPedalState;
	}
	
} // namespace Audio
