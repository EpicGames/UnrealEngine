// Copyright Epic Games, Inc. All Rights Reserved.

#include "SourceEffects/SourceEffectBitCrusher.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(SourceEffectBitCrusher)


void FSourceEffectBitCrusher::Init(const FSoundEffectSourceInitData& InitData)
{
	bIsActive = true;
	BitCrusher.Init(InitData.SampleRate, InitData.NumSourceChannels);

	if (USourceEffectBitCrusherPreset* ProcPreset = Cast<USourceEffectBitCrusherPreset>(Preset.Get()))
	{
		BitMod.Init(InitData.AudioDeviceId, FName("BitDepth"), false /* bInIsBuffered */);
		SampleRateMod.Init(InitData.AudioDeviceId, FName("SampleRate"), false /* bInIsBuffered */);

		auto TransformObjectPtrSet = [](const TSet<TObjectPtr<USoundModulatorBase>>& InPtrs)
		{
			TSet<USoundModulatorBase*> Objects;
			Algo::Transform(InPtrs, Objects, [](const TObjectPtr<USoundModulatorBase>& Obj) { return Obj; });
			return Objects;
		};

		SetBitModulators(TransformObjectPtrSet(ProcPreset->Settings.BitModulation.Modulators));
		SetSampleRateModulators(TransformObjectPtrSet(ProcPreset->Settings.SampleRateModulation.Modulators));
	}
}

void FSourceEffectBitCrusher::OnPresetChanged()
{
	GET_EFFECT_SETTINGS(SourceEffectBitCrusher);

	SettingsCopy = Settings;
}

void FSourceEffectBitCrusher::ProcessAudio(const FSoundEffectSourceInputData& InData, float* OutAudioBufferData)
{
	if (BitMod.ProcessControl(SettingsCopy.BitModulation.Value))
	{
		BitCrusher.SetBitDepthCrush(BitMod.GetValue());
	}

	if (SampleRateMod.ProcessControl(SettingsCopy.SampleRateModulation.Value))
	{
		BitCrusher.SetSampleRateCrush(SampleRateMod.GetValue());
	}

	BitCrusher.ProcessAudio(InData.InputSourceEffectBufferPtr, InData.NumSamples, OutAudioBufferData);
}

void FSourceEffectBitCrusher::SetBitModulator(const USoundModulatorBase* InModulator)
{
	BitMod.SetModulators({ InModulator });
}

void FSourceEffectBitCrusher::SetBitModulators(const TSet<USoundModulatorBase*>& InModulators)
{
	BitMod.SetModulators(InModulators);
}

void FSourceEffectBitCrusher::SetSampleRateModulator(const USoundModulatorBase* InModulator)
{
	SampleRateMod.SetModulators({ InModulator });
}

void FSourceEffectBitCrusher::SetSampleRateModulators(const TSet<USoundModulatorBase*>& InModulators)
{
	SampleRateMod.SetModulators(InModulators);
}

void USourceEffectBitCrusherPreset::OnInit()
{
	auto TransformObjectPtrSet = [](const TSet<TObjectPtr<USoundModulatorBase>>& InPtrs)
	{
		TSet<USoundModulatorBase*> Objects;
		Algo::Transform(InPtrs, Objects, [](const TObjectPtr<USoundModulatorBase>& Obj) { return Obj; });
		return Objects;
	};

	SetBitModulators(TransformObjectPtrSet(Settings.BitModulation.Modulators));
	SetSampleRateModulators(TransformObjectPtrSet(Settings.SampleRateModulation.Modulators));
}

void USourceEffectBitCrusherPreset::SetBits(float InBits)
{
	UpdateSettings([NewBits = InBits](FSourceEffectBitCrusherSettings& OutSettings)
	{
		OutSettings.BitModulation.Value = NewBits;
	});
}

void USourceEffectBitCrusherPreset::SetBitModulator(const USoundModulatorBase* InModulator)
{
	IterateEffects<FSourceEffectBitCrusher>([InModulator](FSourceEffectBitCrusher& InCrusher)
	{
		InCrusher.SetBitModulator(InModulator);
	});
}

void USourceEffectBitCrusherPreset::SetBitModulators(const TSet<USoundModulatorBase*>& InModulators)
{
	IterateEffects<FSourceEffectBitCrusher>([&InModulators](FSourceEffectBitCrusher& InCrusher)
	{
		InCrusher.SetBitModulators(InModulators);
	});
}

void USourceEffectBitCrusherPreset::SetSampleRate(float InSampleRate)
{
	UpdateSettings([NewSampleRate = InSampleRate](FSourceEffectBitCrusherSettings& OutSettings)
	{
		OutSettings.SampleRateModulation.Value = NewSampleRate;
	});
}

void USourceEffectBitCrusherPreset::SetSampleRateModulator(const USoundModulatorBase* InModulator)
{
	IterateEffects<FSourceEffectBitCrusher>([InModulator](FSourceEffectBitCrusher& InCrusher)
	{
		InCrusher.SetSampleRateModulator(InModulator);
	});
}

void USourceEffectBitCrusherPreset::SetSampleRateModulators(const TSet<USoundModulatorBase*>& InModulators)
{
	IterateEffects<FSourceEffectBitCrusher>([&InModulators](FSourceEffectBitCrusher& InCrusher)
	{
		InCrusher.SetSampleRateModulators(InModulators);
	});
}

void USourceEffectBitCrusherPreset::SetSettings(const FSourceEffectBitCrusherBaseSettings& InSettings)
{
	UpdateSettings([NewBaseSettings = InSettings](FSourceEffectBitCrusherSettings& OutSettings)
	{
		OutSettings.BitModulation.Value = NewBaseSettings.BitDepth;
		OutSettings.SampleRateModulation.Value = NewBaseSettings.SampleRate;
	});
}

void USourceEffectBitCrusherPreset::SetModulationSettings(const FSourceEffectBitCrusherSettings& InModulationSettings)
{
	UpdateSettings(InModulationSettings);

	// Must be called to update modulators
	OnInit();
}
