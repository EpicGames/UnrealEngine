// Copyright Epic Games, Inc. All Rights Reserved.

#include "SourceEffects/SourceEffectChorus.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(SourceEffectChorus)

void FSourceEffectChorus::Init(const FSoundEffectSourceInitData& InitData)
{
	bIsActive = true;

	if (USourceEffectChorusPreset* ProcPreset = Cast<USourceEffectChorusPreset>(Preset.Get()))
	{
		Audio::FDeviceId DeviceId = static_cast<Audio::FDeviceId>(InitData.AudioDeviceId);
		const bool bIsBuffered = true;
		DepthMod.Init(DeviceId, bIsBuffered);
		FeedbackMod.Init(DeviceId, bIsBuffered);
		FrequencyMod.Init(DeviceId, FName("FilterFrequency"), bIsBuffered);
		WetMod.Init(DeviceId, bIsBuffered);
		DryMod.Init(DeviceId, bIsBuffered);
		SpreadMod.Init(DeviceId, bIsBuffered);

		auto TransformObjectPtrSet = [](const TSet<TObjectPtr<USoundModulatorBase>>& InPtrs)
		{
			TSet<USoundModulatorBase*> Objects;
			Algo::Transform(InPtrs, Objects, [](const TObjectPtr<USoundModulatorBase>& Obj) { return Obj; });
			return Objects;
		};

		SetDepthModulators(TransformObjectPtrSet(ProcPreset->Settings.DepthModulation.Modulators));
		SetFeedbackModulators(TransformObjectPtrSet(ProcPreset->Settings.FeedbackModulation.Modulators));
		SetFrequencyModulators(TransformObjectPtrSet(ProcPreset->Settings.FrequencyModulation.Modulators));
		SetWetModulators(TransformObjectPtrSet(ProcPreset->Settings.WetModulation.Modulators));
		SetDryModulators(TransformObjectPtrSet(ProcPreset->Settings.DryModulation.Modulators));
		SetSpreadModulators(TransformObjectPtrSet(ProcPreset->Settings.SpreadModulation.Modulators));
	}

	Chorus.Init(InitData.SampleRate, InitData.NumSourceChannels, 2.0f, 64);
}

void FSourceEffectChorus::OnPresetChanged()
{
	GET_EFFECT_SETTINGS(SourceEffectChorus);

	SettingsCopy = Settings;
}

void FSourceEffectChorus::ProcessAudio(const FSoundEffectSourceInputData& InData, float* OutAudioBufferData)
{
	bool bModulated = false;

	bModulated |= DepthMod.ProcessControl(SettingsCopy.DepthModulation.Value, InData.NumSamples);
	bModulated |= FeedbackMod.ProcessControl(SettingsCopy.FeedbackModulation.Value, InData.NumSamples);
	bModulated |= FrequencyMod.ProcessControl(SettingsCopy.FrequencyModulation.Value, InData.NumSamples);
	bModulated |= WetMod.ProcessControl(SettingsCopy.WetModulation.Value, InData.NumSamples);
	bModulated |= DryMod.ProcessControl(SettingsCopy.DryModulation.Value, InData.NumSamples);
	bModulated |= SpreadMod.ProcessControl(SettingsCopy.SpreadModulation.Value, InData.NumSamples);

	if (bModulated)
	{
		const int32 NumChannels = Chorus.GetNumChannels();
		for (int32 SampleIndex = 0; SampleIndex < InData.NumSamples; SampleIndex += NumChannels)
		{
			Chorus.SetWetLevel(WetMod.GetBuffer()[SampleIndex]);
			Chorus.SetDryLevel(DryMod.GetBuffer()[SampleIndex]);
			Chorus.SetSpread(SpreadMod.GetBuffer()[SampleIndex]);

			const float Depth = DepthMod.GetBuffer()[SampleIndex];
			Chorus.SetDepth(Audio::EChorusDelays::Left, Depth);
			Chorus.SetDepth(Audio::EChorusDelays::Center, Depth);
			Chorus.SetDepth(Audio::EChorusDelays::Right, Depth);

			const float Feedback = FeedbackMod.GetBuffer()[SampleIndex];
			Chorus.SetFeedback(Audio::EChorusDelays::Left, Feedback);
			Chorus.SetFeedback(Audio::EChorusDelays::Center, Feedback);
			Chorus.SetFeedback(Audio::EChorusDelays::Right, Feedback);

			const float Frequency = FrequencyMod.GetBuffer()[SampleIndex];
			Chorus.SetFrequency(Audio::EChorusDelays::Left, Frequency);
			Chorus.SetFrequency(Audio::EChorusDelays::Center, Frequency);
			Chorus.SetFrequency(Audio::EChorusDelays::Right, Frequency);

			Chorus.ProcessAudioFrame(&InData.InputSourceEffectBufferPtr[SampleIndex], &OutAudioBufferData[SampleIndex]);
		}
	}
	else
	{
		Chorus.ProcessAudio(InData.InputSourceEffectBufferPtr, InData.NumSamples, OutAudioBufferData);
	}
}

void FSourceEffectChorus::SetDepthModulator(const USoundModulatorBase* InModulator)
{
	DepthMod.SetModulators({ InModulator });
}

void FSourceEffectChorus::SetFeedbackModulator(const USoundModulatorBase* InModulator)
{
	FeedbackMod.SetModulators({ InModulator });
}

void FSourceEffectChorus::SetSpreadModulator(const USoundModulatorBase* InModulator)
{
	SpreadMod.SetModulators({ InModulator });
}

void FSourceEffectChorus::SetDryModulator(const USoundModulatorBase* InModulator)
{
	DryMod.SetModulators({ InModulator });
}

void FSourceEffectChorus::SetWetModulator(const USoundModulatorBase* InModulator)
{
	WetMod.SetModulators({ InModulator });
}

void FSourceEffectChorus::SetFrequencyModulator(const USoundModulatorBase* InModulator)
{
	FrequencyMod.SetModulators({ InModulator });
}

void FSourceEffectChorus::SetDepthModulators(const TSet<USoundModulatorBase*>& InModulators)
{
	DepthMod.SetModulators(InModulators);
}

void FSourceEffectChorus::SetFeedbackModulators(const TSet<USoundModulatorBase*>& InModulators)
{
	FeedbackMod.SetModulators(InModulators);
}

void FSourceEffectChorus::SetSpreadModulators(const TSet<USoundModulatorBase*>& InModulators)
{
	SpreadMod.SetModulators(InModulators);
}

void FSourceEffectChorus::SetDryModulators(const TSet<USoundModulatorBase*>& InModulators)
{
	DryMod.SetModulators(InModulators);
}

void FSourceEffectChorus::SetWetModulators(const TSet<USoundModulatorBase*>& InModulators)
{
	WetMod.SetModulators(InModulators);
}

void FSourceEffectChorus::SetFrequencyModulators(const TSet<USoundModulatorBase*>& InModulators)
{
	FrequencyMod.SetModulators(InModulators);
}

void USourceEffectChorusPreset::OnInit()
{
	auto TransformObjectPtrSet = [](const TSet<TObjectPtr<USoundModulatorBase>>& InPtrs)
	{
		TSet<USoundModulatorBase*> Objects;
		Algo::Transform(InPtrs, Objects, [] (const TObjectPtr<USoundModulatorBase>& Obj) { return Obj; });
		return Objects;
	};

	SetDepthModulators(TransformObjectPtrSet(Settings.DepthModulation.Modulators));
	SetDryModulators(TransformObjectPtrSet(Settings.DryModulation.Modulators));
	SetWetModulators(TransformObjectPtrSet(Settings.WetModulation.Modulators));
	SetFeedbackModulators(TransformObjectPtrSet(Settings.FeedbackModulation.Modulators));
	SetFrequencyModulators(TransformObjectPtrSet(Settings.FrequencyModulation.Modulators));
	SetSpreadModulators(TransformObjectPtrSet(Settings.SpreadModulation.Modulators));
}

void USourceEffectChorusPreset::SetDepth(float InDepth)
{
	UpdateSettings([NewDepth = InDepth](FSourceEffectChorusSettings& OutSettings)
	{
		OutSettings.DepthModulation.Value = NewDepth;
	});
}

void USourceEffectChorusPreset::SetDepthModulator(const USoundModulatorBase* InModulator)
{
	IterateEffects<FSourceEffectChorus>([InModulator](FSourceEffectChorus& InDelay)
	{
		InDelay.SetDepthModulator(InModulator);
	});
}

void USourceEffectChorusPreset::SetDepthModulators(const TSet<USoundModulatorBase*>& Modulators)
{
	IterateEffects<FSourceEffectChorus>([&Modulators](FSourceEffectChorus& InDelay)
	{
		InDelay.SetDepthModulators(Modulators);
	});
}

void USourceEffectChorusPreset::SetFeedback(float InFeedback)
{
	UpdateSettings([NewFeedback = InFeedback](FSourceEffectChorusSettings& OutSettings)
	{
		OutSettings.FeedbackModulation.Value = NewFeedback;
	});
}

void USourceEffectChorusPreset::SetFeedbackModulator(const USoundModulatorBase* InModulator)
{
	IterateEffects<FSourceEffectChorus>([InModulator](FSourceEffectChorus& InDelay)
	{
		InDelay.SetFeedbackModulator(InModulator);
	});
}

void USourceEffectChorusPreset::SetFeedbackModulators(const TSet<USoundModulatorBase*>& Modulators)
{
	IterateEffects<FSourceEffectChorus>([&Modulators](FSourceEffectChorus& InDelay)
	{
		InDelay.SetFeedbackModulators(Modulators);
	});
}

void USourceEffectChorusPreset::SetFrequency(float InFrequency)
{
	UpdateSettings([NewFrequency = InFrequency](FSourceEffectChorusSettings& OutSettings)
	{
		OutSettings.FrequencyModulation.Value = NewFrequency;
	});
}

void USourceEffectChorusPreset::SetFrequencyModulator(const USoundModulatorBase* InModulator)
{
	IterateEffects<FSourceEffectChorus>([InModulator](FSourceEffectChorus& InDelay)
	{
		InDelay.SetFrequencyModulator(InModulator);
	});
}

void USourceEffectChorusPreset::SetFrequencyModulators(const TSet<USoundModulatorBase*>& Modulators)
{
	IterateEffects<FSourceEffectChorus>([&Modulators](FSourceEffectChorus& InDelay)
	{
		InDelay.SetFrequencyModulators(Modulators);
	});
}

void USourceEffectChorusPreset::SetWet(float InWet)
{
	UpdateSettings([NewWet = InWet](FSourceEffectChorusSettings& OutSettings)
	{
		OutSettings.WetModulation.Value = NewWet;
	});
}

void USourceEffectChorusPreset::SetWetModulator(const USoundModulatorBase* InModulator)
{
	IterateEffects<FSourceEffectChorus>([InModulator](FSourceEffectChorus& InDelay)
	{
		InDelay.SetWetModulator(InModulator);
	});
}

void USourceEffectChorusPreset::SetWetModulators(const TSet<USoundModulatorBase*>& Modulators)
{
	IterateEffects<FSourceEffectChorus>([&Modulators](FSourceEffectChorus& InDelay)
	{
		InDelay.SetWetModulators(Modulators);
	});
}

void USourceEffectChorusPreset::SetDry(float InDry)
{
	UpdateSettings([NewDry = InDry](FSourceEffectChorusSettings& OutSettings)
	{
		OutSettings.DryModulation.Value = NewDry;
	});
}

void USourceEffectChorusPreset::SetDryModulator(const USoundModulatorBase* InModulator)
{
	IterateEffects<FSourceEffectChorus>([InModulator](FSourceEffectChorus& InDelay)
	{
		InDelay.SetDryModulator(InModulator);
	});
}

void USourceEffectChorusPreset::SetDryModulators(const TSet<USoundModulatorBase*>& Modulators)
{
	IterateEffects<FSourceEffectChorus>([&Modulators](FSourceEffectChorus& InDelay)
	{
		InDelay.SetDryModulators(Modulators);
	});
}

void USourceEffectChorusPreset::SetSpread(float InSpread)
{
	UpdateSettings([NewSpread = InSpread](FSourceEffectChorusSettings& OutSettings)
	{
		OutSettings.SpreadModulation.Value = NewSpread;
	});
}

void USourceEffectChorusPreset::SetSpreadModulator(const USoundModulatorBase* InModulator)
{
	IterateEffects<FSourceEffectChorus>([InModulator](FSourceEffectChorus& InDelay)
	{
		InDelay.SetSpreadModulator(InModulator);
	});
}

void USourceEffectChorusPreset::SetSpreadModulators(const TSet<USoundModulatorBase*>& Modulators)
{
	IterateEffects<FSourceEffectChorus>([&Modulators](FSourceEffectChorus& InDelay)
	{
		InDelay.SetSpreadModulators(Modulators);
	});
}

void USourceEffectChorusPreset::SetSettings(const FSourceEffectChorusBaseSettings& InSettings)
{
	UpdateSettings([NewBaseSettings = InSettings](FSourceEffectChorusSettings& OutSettings)
	{
		OutSettings.DepthModulation.Value = NewBaseSettings.Depth;
		OutSettings.FrequencyModulation.Value = NewBaseSettings.Frequency;
		OutSettings.FeedbackModulation.Value = NewBaseSettings.Feedback;
		OutSettings.WetModulation.Value = NewBaseSettings.WetLevel;
		OutSettings.DryModulation.Value = NewBaseSettings.DryLevel;
		OutSettings.SpreadModulation.Value = NewBaseSettings.Spread;
	});
}

void USourceEffectChorusPreset::SetModulationSettings(const FSourceEffectChorusSettings& InModulationSettings)
{
	UpdateSettings(InModulationSettings);

	// Must be called to update modulators
	OnInit();
}
