// Copyright Epic Games, Inc. All Rights Reserved.

#include "SourceEffects/SourceEffectConvolutionReverb.h"

#include "AudioMixerNullDevice.h"
#include "DSP/AlignedBlockBuffer.h" // IWYU pragma: keep
#include "DSP/FloatArrayMath.h"
#include "EffectConvolutionReverb.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SourceEffectConvolutionReverb)

FSourceEffectConvolutionReverbSettings::FSourceEffectConvolutionReverbSettings()
	: NormalizationVolumeDb(-24.f)
	, WetVolumeDb(0.f)
	, DryVolumeDb(-96.f)
	, bBypass(false)
{
}

FSourceEffectConvolutionReverb::FSourceEffectConvolutionReverb(const USourceEffectConvolutionReverbPreset* InPreset)
	: Reverb(MakeShared<Audio::FEffectConvolutionReverb, ESPMode::ThreadSafe>())
{
	UpdateConvolutionReverb(InPreset);

	if (nullptr != InPreset)
	{
		SetConvolutionReverbParameters(InPreset->GetSettings());
	}
}

FSourceEffectConvolutionReverb::~FSourceEffectConvolutionReverb()
{
}

void FSourceEffectConvolutionReverb::Init(const FSoundEffectSourceInitData& InitData)
{
	using namespace Audio;

	NumChannels = InitData.NumSourceChannels;

	Reverb->SetSampleRate(InitData.SampleRate);
	Reverb->UpdateVersion();
	Reverb->Init();
	Reverb->UpdateChannelCount(NumChannels, NumChannels);
}


void FSourceEffectConvolutionReverb::OnPresetChanged()
{
	USourceEffectConvolutionReverbPreset* ConvolutionPreset = CastChecked<USourceEffectConvolutionReverbPreset>(Preset);
	FSourceEffectConvolutionReverbSettings Settings = ConvolutionPreset->GetSettings();

	// Copy settings from FSourceEffectConvolutionReverbSettings needed for FConvolutionReverbSettings 
	// FConvolutionReverbSettings represents runtime settings which do not need the 
	// FConvlutionReverb object to be rebuilt. Some settings in FSourceEffectConvolutionReverbSettings
	// force a rebuild of FConvolutionReverb-> Those are handled in USourceEffectConvolutinReverbPreset::PostEditChangeProperty
	SetConvolutionReverbParameters(Settings);
}

void FSourceEffectConvolutionReverb::SetConvolutionReverbParameters(const FSourceEffectConvolutionReverbSettings& InSettings)
{
	Audio::FConvolutionReverbSettings ReverbSettings;

	ReverbSettings.NormalizationVolume = Audio::ConvertToLinear(InSettings.NormalizationVolumeDb);

	WetVolume = Audio::ConvertToLinear(InSettings.WetVolumeDb);
	DryVolume = Audio::ConvertToLinear(InSettings.DryVolumeDb);

	Reverb->SetSettings(ReverbSettings);
	Reverb->SetBypass(InSettings.bBypass);
}

void FSourceEffectConvolutionReverb::ProcessAudio(const FSoundEffectSourceInputData& InData, float* OutAudioBufferData)
{
	using namespace Audio;

	check(nullptr != InData.InputSourceEffectBufferPtr);
	check(nullptr != OutAudioBufferData);
	check(NumChannels != 0);

	const int32 NumFrames = InData.NumSamples / NumChannels;

	Reverb->UpdateChannelCount(NumChannels, NumChannels);

	Reverb->ProcessAudio(NumChannels, InData.InputSourceEffectBufferPtr, NumChannels, OutAudioBufferData, NumFrames);

	TArrayView<float> OutAudioBufferView(OutAudioBufferData, InData.NumSamples);
	TArrayView<const float> InputSourceEffectBufferView(InData.InputSourceEffectBufferPtr, InData.NumSamples);

	// Process Wet/Dry mix
	if (FMath::IsNearlyEqual(WetVolume, 1.f) == false)
	{
		Audio::ArrayMultiplyByConstantInPlace(OutAudioBufferView, WetVolume);
	}
	if (DryVolume > KINDA_SMALL_NUMBER)
	{
		Audio::ArrayMixIn(InputSourceEffectBufferView, OutAudioBufferView, DryVolume);
	}
}

AudioConvReverbIntrinsics::FVersionData FSourceEffectConvolutionReverb::UpdateConvolutionReverb(const USourceEffectConvolutionReverbPreset* InPreset)
{
	using namespace Audio;

	// Copy data from preset into internal FConvolutionReverbInitData
	// Reset data
	FConvolutionReverbInitData InitData = FConvolutionReverbInitData();

	if (nullptr != InPreset)
	{
		if (InPreset->ImpulseResponse)
		{
			UAudioImpulseResponse* IR = InPreset->ImpulseResponse;

			InitData.Samples = IR->ImpulseResponse;
			InitData.AlgorithmSettings.NumImpulseResponses = IR->NumChannels;
			InitData.ImpulseSampleRate = IR->SampleRate;
			InitData.bIsImpulseTrueStereo = IR->bTrueStereo && (IR->NumChannels % 2 == 0);
			InitData.NormalizationVolume = ConvertToLinear(IR->NormalizationVolumeDb);
		}

		switch (InPreset->BlockSize)
		{
		case ESubmixEffectConvolutionReverbBlockSize::BlockSize256:
			InitData.AlgorithmSettings.BlockNumSamples = 256;
			break;

		case ESubmixEffectConvolutionReverbBlockSize::BlockSize512:
			InitData.AlgorithmSettings.BlockNumSamples = 512;
			break;

		case ESubmixEffectConvolutionReverbBlockSize::BlockSize1024:
		default:
			InitData.AlgorithmSettings.BlockNumSamples = 1024;
			break;
		}

		InitData.AlgorithmSettings.bEnableHardwareAcceleration = InPreset->bEnableHardwareAcceleration;
	}

	Reverb->SetInitData(InitData);
	return Reverb->UpdateVersion();
}

void FSourceEffectConvolutionReverb::RebuildConvolutionReverb()
{
	Reverb->BuildReverb();
}

/********************************************************************/
/***************** USourceEffectConvolutionReverbPreset *************/
/********************************************************************/

USourceEffectConvolutionReverbPreset::USourceEffectConvolutionReverbPreset(const FObjectInitializer& ObjectInitializer)
:	Super(ObjectInitializer)
,	ImpulseResponse(nullptr)
,	BlockSize(ESubmixEffectConvolutionReverbBlockSize::BlockSize1024)
,	bEnableHardwareAcceleration(true)
{}

bool USourceEffectConvolutionReverbPreset::CanFilter() const 
{
	return false; 
}

bool USourceEffectConvolutionReverbPreset::HasAssetActions() const 
{ 
	return true; 
}

FText USourceEffectConvolutionReverbPreset::GetAssetActionName() const 
{ 
	return FText::FromString(TEXT("SourceEffectConvolutionReverb")); 
}

UClass* USourceEffectConvolutionReverbPreset::GetSupportedClass() const 
{ 
	return USourceEffectConvolutionReverbPreset::StaticClass(); 
}

FSoundEffectBase* USourceEffectConvolutionReverbPreset::CreateNewEffect() const 
{ 
	// Pass a pointer to self into this constructor.  This enables the effect to have
	// a copy of the impulse response data. The effect can then prepare the impulse
	// response during it's "Init" routine.
	return new FSourceEffectConvolutionReverb(this); 
}

USoundEffectPreset* USourceEffectConvolutionReverbPreset::CreateNewPreset(UObject* InParent, FName Name, EObjectFlags Flags) const
{ 
	USoundEffectPreset* NewPreset = NewObject<USourceEffectConvolutionReverbPreset>(InParent, GetSupportedClass(), Name, Flags); 
	NewPreset->Init(); 
	return NewPreset; 
}

void USourceEffectConvolutionReverbPreset::Init() 
{ 
	FScopeLock ScopeLock(&SettingsCritSect);
	SettingsCopy = Settings;
}

void USourceEffectConvolutionReverbPreset::UpdateSettings()
{
	{
		// Copy settings to audio-render-thread version
		FScopeLock ScopeLock(&SettingsCritSect);
		SettingsCopy = Settings; 
	}
	// This sets the 'bChanged' to true on related effect instances which will 
	// trigger a OnPresetChanged call on the audio render thread.
	Update(); 
} 

FSourceEffectConvolutionReverbSettings USourceEffectConvolutionReverbPreset::GetSettings() const
{ 
	FScopeLock ScopeLock(&SettingsCritSect);
	return SettingsCopy; 
} 

void USourceEffectConvolutionReverbPreset::SetSettings(const FSourceEffectConvolutionReverbSettings& InSettings)
{
	Settings = InSettings;

	if (nullptr != ImpulseResponse)
	{
		SetImpulseResponseSettings(ImpulseResponse);
	}

	UpdateSettings();
}

void USourceEffectConvolutionReverbPreset::SetImpulseResponse(UAudioImpulseResponse* InImpulseResponse)
{
	ImpulseResponse = InImpulseResponse;
	SetImpulseResponseSettings(InImpulseResponse);
	RebuildConvolutionReverb();
}

void USourceEffectConvolutionReverbPreset::RebuildConvolutionReverb()
{
	using namespace AudioConvReverbIntrinsics;
	using FEffectWeakPtr = TWeakPtr<FSoundEffectBase, ESPMode::ThreadSafe>;
	using FEffectSharedPtr = TSharedPtr<FSoundEffectBase, ESPMode::ThreadSafe>;
	using FSourceEffectConvSharedPtr = TSharedPtr<FSourceEffectConvolutionReverb, ESPMode::ThreadSafe>;

	// Iterator over effects and update the convolution Reverb->
	for (FEffectWeakPtr& InstanceWeakPtr : Instances)
	{
		FEffectSharedPtr InstanceSharedPtr = InstanceWeakPtr.Pin();
		if (InstanceSharedPtr.IsValid())
		{
			FSourceEffectConvSharedPtr ConvEffectSharedPtr = StaticCastSharedPtr<FSourceEffectConvolutionReverb>(InstanceSharedPtr);

			FVersionData VersionData = ConvEffectSharedPtr->UpdateConvolutionReverb(this);
			ConvEffectSharedPtr->RebuildConvolutionReverb();
		}
	}
}

#if WITH_EDITORONLY_DATA
void USourceEffectConvolutionReverbPreset::BindToImpulseResponseObjectChange()
{
	if (nullptr != ImpulseResponse)
	{
		if (!DelegateHandles.Contains(ImpulseResponse))
		{
			FDelegateHandle Handle = ImpulseResponse->OnObjectPropertyChanged.AddUObject(this, &USourceEffectConvolutionReverbPreset::PostEditChangeImpulseProperty);
			if (Handle.IsValid())
			{
				DelegateHandles.Add(ImpulseResponse, Handle);
			}
		}
	}
}

void USourceEffectConvolutionReverbPreset::PreEditChange(FProperty* PropertyAboutToChange)
{
	Super::PreEditChange(PropertyAboutToChange);

	if (nullptr == PropertyAboutToChange)
	{
		return;
	}

	const FName ImpulseResponseFName = GET_MEMBER_NAME_CHECKED(USourceEffectConvolutionReverbPreset, ImpulseResponse);

	if (PropertyAboutToChange->GetFName() == ImpulseResponseFName)
	{
		if (nullptr == ImpulseResponse)
		{
			return;
		}

		if (DelegateHandles.Contains(ImpulseResponse))
		{
			FDelegateHandle DelegateHandle = DelegateHandles[ImpulseResponse];

			if (DelegateHandle.IsValid())
			{
				ImpulseResponse->OnObjectPropertyChanged.Remove(DelegateHandle);
			}

			DelegateHandles.Remove(ImpulseResponse);
		}
	}
}



void USourceEffectConvolutionReverbPreset::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName ImpulseResponseFName = GET_MEMBER_NAME_CHECKED(USourceEffectConvolutionReverbPreset, ImpulseResponse);

	// If any of these properties are updated, the FConvolutionReverb object needs
	// to be rebuilt.
	const TArray<FName> ConvolutionReverbProperties(
		{
			ImpulseResponseFName,
			GET_MEMBER_NAME_CHECKED(USourceEffectConvolutionReverbPreset, bEnableHardwareAcceleration),
			GET_MEMBER_NAME_CHECKED(USourceEffectConvolutionReverbPreset, BlockSize),
		}
	);

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		if (FProperty* PropertyThatChanged = PropertyChangedEvent.Property)
		{
			const FName& Name = PropertyThatChanged->GetFName();

			// Need to update impulse reponse bindings and normalization before rebuilding convolution reverb
			if (Name == ImpulseResponseFName)
			{
				BindToImpulseResponseObjectChange();
				
				if (nullptr != ImpulseResponse)
				{
					SetImpulseResponseSettings(ImpulseResponse);
				}
			}

			if (ConvolutionReverbProperties.Contains(Name))
			{
				RebuildConvolutionReverb();
			}
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void USourceEffectConvolutionReverbPreset::PostEditChangeImpulseProperty(FPropertyChangedEvent& PropertyChangedEvent) 
{
	const FName NormalizationVolumeFName = GET_MEMBER_NAME_CHECKED(UAudioImpulseResponse, NormalizationVolumeDb);
	const FName TrueStereoFName = GET_MEMBER_NAME_CHECKED(UAudioImpulseResponse, bTrueStereo);

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		if (FProperty* PropertyThatChanged = PropertyChangedEvent.Property)
		{
			const FName& Name = PropertyThatChanged->GetFName();

			if (Name == NormalizationVolumeFName)
			{
				if (nullptr != ImpulseResponse)
				{
					Settings.NormalizationVolumeDb = ImpulseResponse->NormalizationVolumeDb;
					UpdateSettings();
				}
			}
			else if (Name == TrueStereoFName)
			{
				RebuildConvolutionReverb();
			}
		}
	}
}

#endif

void USourceEffectConvolutionReverbPreset::SetImpulseResponseSettings(UAudioImpulseResponse* InImpulseResponse)
{
	if (nullptr != InImpulseResponse)
	{
		// Set this value, but do not call UpdateSettings(). UpdateSettings() is handled elsewhere.
		Settings.NormalizationVolumeDb = InImpulseResponse->NormalizationVolumeDb;
	}
}

void USourceEffectConvolutionReverbPreset::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	// Bind to trigger new convolution algorithms when UAudioImpulseResponse object changes.
	BindToImpulseResponseObjectChange();
#endif

	SetImpulseResponseSettings(ImpulseResponse);
}

