// Copyright Epic Games, Inc. All Rights Reserved.

#include "SourceEffects/SourceEffectDynamicsProcessor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SourceEffectDynamicsProcessor)


FSourceEffectDynamicsProcessor::FSourceEffectDynamicsProcessor()
{
}

void FSourceEffectDynamicsProcessor::Init(const FSoundEffectSourceInitData& InitData)
{
	bIsActive = true;
	DynamicsProcessor.Init(InitData.SampleRate, InitData.NumSourceChannels);
}

void FSourceEffectDynamicsProcessor::OnPresetChanged()
{
	GET_EFFECT_SETTINGS(SourceEffectDynamicsProcessor);

	switch (Settings.DynamicsProcessorType)
	{
		default:
		case ESourceEffectDynamicsProcessorType::Compressor:
			DynamicsProcessor.SetProcessingMode(Audio::EDynamicsProcessingMode::Compressor);
			break;

		case ESourceEffectDynamicsProcessorType::Limiter:
			DynamicsProcessor.SetProcessingMode(Audio::EDynamicsProcessingMode::Limiter);
			break;

		case ESourceEffectDynamicsProcessorType::Expander:
			DynamicsProcessor.SetProcessingMode(Audio::EDynamicsProcessingMode::Expander);
			break;

		case ESourceEffectDynamicsProcessorType::Gate:
			DynamicsProcessor.SetProcessingMode(Audio::EDynamicsProcessingMode::Gate);
			break;

		case ESourceEffectDynamicsProcessorType::UpwardsCompressor:
			DynamicsProcessor.SetProcessingMode(Audio::EDynamicsProcessingMode::UpwardsCompressor);
			break;
	}

	switch (Settings.PeakMode)
	{
		default:
		case ESourceEffectDynamicsPeakMode::MeanSquared:
			DynamicsProcessor.SetPeakMode(Audio::EPeakMode::MeanSquared);
			break;

		case ESourceEffectDynamicsPeakMode::RootMeanSquared:
			DynamicsProcessor.SetPeakMode(Audio::EPeakMode::RootMeanSquared);
			break;

		case ESourceEffectDynamicsPeakMode::Peak:
			DynamicsProcessor.SetPeakMode(Audio::EPeakMode::Peak);
			break;
	}

	DynamicsProcessor.SetLookaheadMsec(Settings.LookAheadMsec);
	DynamicsProcessor.SetAttackTime(Settings.AttackTimeMsec);
	DynamicsProcessor.SetReleaseTime(Settings.ReleaseTimeMsec);
	DynamicsProcessor.SetThreshold(Settings.ThresholdDb);
	DynamicsProcessor.SetRatio(Settings.Ratio);
	DynamicsProcessor.SetKneeBandwidth(Settings.KneeBandwidthDb);
	DynamicsProcessor.SetInputGain(Settings.InputGainDb);
	DynamicsProcessor.SetOutputGain(Settings.OutputGainDb);
	DynamicsProcessor.SetAnalogMode(Settings.bAnalogMode);

	DynamicsProcessor.SetChannelLinkMode(Settings.bStereoLinked ? Audio::EDynamicsProcessorChannelLinkMode::Average : Audio::EDynamicsProcessorChannelLinkMode::Disabled);
}

void FSourceEffectDynamicsProcessor::ProcessAudio(const FSoundEffectSourceInputData& InData, float* OutAudioBufferData)
{
	DynamicsProcessor.ProcessAudio(InData.InputSourceEffectBufferPtr, InData.NumSamples, OutAudioBufferData);
}

void USourceEffectDynamicsProcessorPreset::SetSettings(const FSourceEffectDynamicsProcessorSettings& InSettings)
{
	UpdateSettings(InSettings);
}
