// Copyright Epic Games, Inc. All Rights Reserved.

#include "SourceEffects/SourceEffectEQ.h"
#include "DSP/Filter.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SourceEffectEQ)

FSourceEffectEQ::FSourceEffectEQ()
	: SampleRate(0)
	, NumChannels(0)
{
	FMemory::Memzero((void*)InAudioFrame, sizeof(float)*2);
	FMemory::Memzero((void*)OutAudioFrame, sizeof(float)*2);
}

void FSourceEffectEQ::Init(const FSoundEffectSourceInitData& InitData)
{
	SampleRate = InitData.SampleRate;
	NumChannels = InitData.NumSourceChannels;
}

void FSourceEffectEQ::OnPresetChanged() 
{
	GET_EFFECT_SETTINGS(SourceEffectEQ);

	// Remove the filter bands at the end of the Settings eq bands is less than what we already have
	const int32 NumSettingBands = Settings.EQBands.Num();
	if (Filters.Num() < NumSettingBands)
	{
		const int32 Delta = NumSettingBands - Filters.Num();
		
		// Add them to the array
		int32 i = Filters.AddDefaulted(Delta);

		// Now initialize the new filters
		for (; i < Filters.Num(); ++i)
		{
			Filters[i].Init(SampleRate, NumChannels, Audio::EBiquadFilter::ParametricEQ);
		}
	}
	else
	{
		// Disable filters if they don't match
		for (int32 i = NumSettingBands; i < Filters.Num(); ++i)
		{
			Filters[i].SetEnabled(false);
		}
	}

	check(Settings.EQBands.Num() <= Filters.Num());

	// Now make sure the filters settings are the same as the eq settings
	for (int32 i = 0; i < NumSettingBands; ++i)
	{
		const FSourceEffectEQBand& EQBandSetting = Settings.EQBands[i];

		Filters[i].SetEnabled(EQBandSetting.bEnabled);
		Filters[i].SetParams(Audio::EBiquadFilter::ParametricEQ, FMath::Max(20.0f, EQBandSetting.Frequency), EQBandSetting.Bandwidth, EQBandSetting.GainDb);
	}
}

void FSourceEffectEQ::ProcessAudio(const FSoundEffectSourceInputData& InData, float* OutAudioBufferData)
{
	float* InAudioBufferData = InData.InputSourceEffectBufferPtr;

	if (Filters.Num() == 0)
	{
		FMemory::Memcpy(OutAudioBufferData, InAudioBufferData, sizeof(float)*InData.NumSamples);
		return;
	}

	for (int32 FilterIndex = 0; FilterIndex < Filters.Num(); ++FilterIndex)
	{
		Filters[FilterIndex].ProcessAudio(InAudioBufferData, InData.NumSamples, InAudioBufferData);
	}

	FMemory::Memcpy(OutAudioBufferData, InAudioBufferData, sizeof(float)*InData.NumSamples);
}

void USourceEffectEQPreset::SetSettings(const FSourceEffectEQSettings& InSettings)
{
	UpdateSettings(InSettings);
}
