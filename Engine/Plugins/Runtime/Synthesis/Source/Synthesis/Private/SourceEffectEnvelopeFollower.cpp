// Copyright Epic Games, Inc. All Rights Reserved.

#include "SourceEffects/SourceEffectEnvelopeFollower.h"
#include "Tickable.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SourceEffectEnvelopeFollower)

class FSourceEffectEnvFollowerNotifier : public FTickableGameObject, public IEnvelopeFollowerNotifier
{
public:
	FSourceEffectEnvFollowerNotifier()
		: NumListenersRegistered(0)
		, NumInstances(0)
	{}

	virtual TStatId GetStatId() const override
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FSourceEffectEnvFollowerNotifier, STATGROUP_Tickables);
	}

	virtual void Tick(float DeltaTime) override
	{
		PumpCommandQueue();

		// Loop through 
		for (auto DataEntry : EnvelopeFollowerData)
		{
			FEnvFollowListenerData& ListenerData = DataEntry.Value;

			if (ListenerData.Listeners.Num() > 0)
			{
				int32 NumListenerDataInstances = ListenerData.InstanceData.Num();
				if (NumListenerDataInstances > 0)
				{
					// Get average of all envelope data for all instances
					float AvgEnvValue = 0.0f;

					for (int32 i = 0; i < ListenerData.InstanceData.Num(); ++i)
					{
						AvgEnvValue += ListenerData.InstanceData[i].EnvelopeValue;
					}
					AvgEnvValue /= (float)NumListenerDataInstances;

					for (UEnvelopeFollowerListener* Listener : ListenerData.Listeners)
					{
						if (IsValid(Listener) && Listener->OnEnvelopeFollowerUpdate.IsBound())
						{
							Listener->OnEnvelopeFollowerUpdate.Broadcast(AvgEnvValue);
						}
					}
				}
			}
		}

	}

	virtual ETickableTickType GetTickableTickType() const override
	{
		return ETickableTickType::Always;
	}

	virtual bool IsTickableWhenPaused() const override
	{
		return true;
	}

	void RegisterEnvelopFollowerListener(uint32 PresetUniqueId, UEnvelopeFollowerListener* EnvFollowerListener)
	{
		++NumListenersRegistered;

		EnvFollowerListener->Init(this, PresetUniqueId);

		FEnvFollowListenerData* Entry = EnvelopeFollowerData.Find(PresetUniqueId);
		if (Entry)
		{
			Entry->Listeners.Add(EnvFollowerListener);
		}
		else
		{
			FEnvFollowListenerData NewEntry;
			NewEntry.Listeners.Add(EnvFollowerListener);
			EnvelopeFollowerData.Add(PresetUniqueId, NewEntry);
		}
	}

	virtual void UnregisterEnvelopeFollowerListener(uint32 PresetUniqueId, UEnvelopeFollowerListener* EnvFollowerListener) override
	{
		FEnvFollowListenerData* Entry = EnvelopeFollowerData.Find(PresetUniqueId);
		if (Entry)
		{
			--NumListenersRegistered;

			Entry->Listeners.Remove(EnvFollowerListener);
		}
	}

	void AddEnvFollowerInstance(uint32 PresetUniqueId, uint32 InstanceId)
	{
		PushToCommandQueue([this, PresetUniqueId, InstanceId]()
		{
			++NumInstances;
			FEnvFollowListenerData* Entry = EnvelopeFollowerData.Find(PresetUniqueId);
			if (Entry)
			{
				Entry->InstanceData.Add(FInstanceData(InstanceId));
			}
			else
			{
				FEnvFollowListenerData NewEntry;
				NewEntry.InstanceData.Add(FInstanceData(InstanceId));
				EnvelopeFollowerData.Add(PresetUniqueId, NewEntry);
			}
		});
	}

	void RemoveEnvFollowerInstance(uint32 PresetUniqueId, uint32 InstanceId)
	{
		PushToCommandQueue([this, PresetUniqueId, InstanceId]()
		{
			ensure(NumInstances > 0);
			--NumInstances;
			FEnvFollowListenerData* Entry = EnvelopeFollowerData.Find(PresetUniqueId);
			if (Entry)
			{
				for (int32 i = 0; i < Entry->InstanceData.Num(); ++i)
				{
					if (Entry->InstanceData[i].InstanceId == InstanceId)
					{
						Entry->InstanceData.RemoveAtSwap(i);
						break;
					}
				}
			}
		});
	}

	void UpdateEnvFollowerInstance(uint32 PresetUniqueId, uint32 InstanceId, float EnvelopeValue)
	{
		PushToCommandQueue([this, PresetUniqueId, InstanceId, EnvelopeValue]()
		{
			// Don't need to do anything if nobody is listening...
			if (NumListenersRegistered > 0)
			{
				FEnvFollowListenerData* Entry = EnvelopeFollowerData.Find(PresetUniqueId);
				if (Entry)
				{
					for (int32 i = 0; i < Entry->InstanceData.Num(); ++i)
					{
						if (Entry->InstanceData[i].InstanceId == InstanceId)
						{
							Entry->InstanceData[i].EnvelopeValue = EnvelopeValue;
							break;
						}
					}
				}
			}
		});
	}

protected:

	void PushToCommandQueue(TFunction<void()> Command)
	{
		CommandQueue.Enqueue(MoveTemp(Command));
	}

	void PumpCommandQueue()
	{
		TFunction<void()> Func;
		while (CommandQueue.Dequeue(Func))
		{
			Func();
		}
	}

	int32 NumListenersRegistered;
	int32 NumInstances;

	struct FInstanceData
	{
		int32 InstanceId;
		float EnvelopeValue;

		FInstanceData(int32 Id)
			: InstanceId(Id)
		{}
	};

	struct FEnvFollowListenerData
	{
		TArray<UEnvelopeFollowerListener*> Listeners;
		TArray<FInstanceData> InstanceData;
	};

	TQueue<TFunction<void()>> CommandQueue;
	TMap<uint32, FEnvFollowListenerData> EnvelopeFollowerData;
};

// Singleton instance
TSharedPtr<FSourceEffectEnvFollowerNotifier> SourceEffectEnvFollowerNotifier;

static void InitEnvelopeFollowerNotifier()
{
	if (!SourceEffectEnvFollowerNotifier.IsValid())
	{
		SourceEffectEnvFollowerNotifier = TSharedPtr<FSourceEffectEnvFollowerNotifier>(new FSourceEffectEnvFollowerNotifier());
	}
}


void FSourceEffectEnvelopeFollower::Init(const FSoundEffectSourceInitData& InitData)
{
	static uint32 GInstanceId = 0;

	InstanceId = GInstanceId++;

	InitEnvelopeFollowerNotifier();

	OwningPresetUniqueId = InitData.ParentPresetUniqueId;

	SourceEffectEnvFollowerNotifier->AddEnvFollowerInstance(OwningPresetUniqueId, InstanceId);

	FramesToNotify = 1024;
	FrameCount = 0;
	bIsActive = true;
	CurrentEnvelopeValue = 0.0f;
	NumChannels = InitData.NumSourceChannels;

	Audio::FInlineEnvelopeFollowerInitParams EnvelopeFollowerInitParams;
	EnvelopeFollowerInitParams.SampleRate = InitData.SampleRate;
	EnvelopeFollower.Init(EnvelopeFollowerInitParams);
}

FSourceEffectEnvelopeFollower::~FSourceEffectEnvelopeFollower()
{
	SourceEffectEnvFollowerNotifier->RemoveEnvFollowerInstance(OwningPresetUniqueId, InstanceId);
}

void FSourceEffectEnvelopeFollower::OnPresetChanged()
{
	GET_EFFECT_SETTINGS(SourceEffectEnvelopeFollower);

	EnvelopeFollower.SetAnalog(Settings.bIsAnalogMode);
	EnvelopeFollower.SetAttackTime(Settings.AttackTime);
	EnvelopeFollower.SetReleaseTime(Settings.ReleaseTime);
	EnvelopeFollower.SetMode((Audio::EPeakMode::Type)Settings.PeakMode);
}

void FSourceEffectEnvelopeFollower::ProcessAudio(const FSoundEffectSourceInputData& InData, float* OutAudioBufferData)
{
	// Memcpy the output (this is a "pass through" effect)
	FMemory::Memcpy(OutAudioBufferData, InData.InputSourceEffectBufferPtr, sizeof(float)*InData.NumSamples);

	for (int32 SampleIndex = 0; SampleIndex < InData.NumSamples; SampleIndex += NumChannels)
	{
		float SampleValue = 0.0f;
		for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ++ChannelIndex)
		{
			SampleValue += InData.InputSourceEffectBufferPtr[SampleIndex + ChannelIndex];
		}

		if (NumChannels == 2)
		{
			SampleValue *= 0.5f;
		}

		CurrentEnvelopeValue = EnvelopeFollower.ProcessSample(SampleValue);

		if ((FrameCount++ & (FramesToNotify - 1)) == 0)
		{
			SourceEffectEnvFollowerNotifier->UpdateEnvFollowerInstance(OwningPresetUniqueId, InstanceId, FMath::Clamp(CurrentEnvelopeValue, 0.f, 1.f));
		}
	}
}

void USourceEffectEnvelopeFollowerPreset::SetSettings(const FSourceEffectEnvelopeFollowerSettings& InSettings)
{
	UpdateSettings(InSettings);
}

void USourceEffectEnvelopeFollowerPreset::RegisterEnvelopeFollowerListener(UEnvelopeFollowerListener* EnvelopeFollowerListener)
{
	InitEnvelopeFollowerNotifier();

	SourceEffectEnvFollowerNotifier->RegisterEnvelopFollowerListener(this->GetUniqueID(), EnvelopeFollowerListener);
}

void USourceEffectEnvelopeFollowerPreset::UnregisterEnvelopeFollowerListener(UEnvelopeFollowerListener* EnvelopeFollowerListener)
{
	InitEnvelopeFollowerNotifier();

	SourceEffectEnvFollowerNotifier->UnregisterEnvelopeFollowerListener(this->GetUniqueID(), EnvelopeFollowerListener);
}

