// Copyright Epic Games, Inc. All Rights Reserved.

#include "SourceEffects/SourceEffectFilter.h"
#include "AudioBusSubsystem.h"
#include "AudioMixerDevice.h"
#include "DSP/BufferVectorOperations.h"
#include "DSP/FloatArrayMath.h"
#include "AudioMixerDevice.h"
#include "Sound/AudioBus.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SourceEffectFilter)

FSourceEffectFilter::FSourceEffectFilter()
	: CurrentFilter(nullptr)
	, SampleRate(0.0f)
	, CutoffFrequency(8000.0f)
	, BaseCutoffFrequency(8000.0f)
	, FilterQ(2.0f)
	, BaseFilterQ(2.0f)
	, CircuitType(ESourceEffectFilterCircuit::StateVariable)
	, FilterType(ESourceEffectFilterType::LowPass)
{
	FMemory::Memzero(AudioInput, 2 * sizeof(float));
	FMemory::Memzero(AudioOutput, 2 * sizeof(float));
}

FSourceEffectFilter::~FSourceEffectFilter()
{

}

void FSourceEffectFilter::Init(const FSoundEffectSourceInitData& InitData)
{
	bIsActive = true;
	NumChannels = InitData.NumSourceChannels;
	StateVariableFilter.Init(InitData.SampleRate, NumChannels);
	LadderFilter.Init(InitData.SampleRate, NumChannels);
	OnePoleFilter.Init(InitData.SampleRate, NumChannels);

	SampleRate = InitData.SampleRate;
	AudioDeviceId = InitData.AudioDeviceId;

	UpdateFilter();
}

void FSourceEffectFilter::UpdateFilter()
{
	switch (CircuitType)
	{
		default:
		case ESourceEffectFilterCircuit::OnePole:
		{
			CurrentFilter = &OnePoleFilter;
		}
		break;

		case ESourceEffectFilterCircuit::StateVariable:
		{
			CurrentFilter = &StateVariableFilter;
		}
		break;

		case ESourceEffectFilterCircuit::Ladder:
		{
			CurrentFilter = &LadderFilter;
		}
		break;
	}

	switch (FilterType)
	{
		default:
		case ESourceEffectFilterType::LowPass:
			CurrentFilter->SetFilterType(Audio::EFilter::LowPass);
			break;

		case ESourceEffectFilterType::HighPass:
			CurrentFilter->SetFilterType(Audio::EFilter::HighPass);
			break;

		case ESourceEffectFilterType::BandPass:
			CurrentFilter->SetFilterType(Audio::EFilter::BandPass);
			break;

		case ESourceEffectFilterType::BandStop:
			CurrentFilter->SetFilterType(Audio::EFilter::BandStop);
			break;
	}

	CurrentFilter->SetFrequency(CutoffFrequency);
	CurrentFilter->SetQ(FilterQ);
	CurrentFilter->Update();
}

void FSourceEffectFilter::OnPresetChanged()
{
	GET_EFFECT_SETTINGS(SourceEffectFilter);

	CircuitType = Settings.FilterCircuit;
	FilterType = Settings.FilterType;
	CutoffFrequency = Settings.CutoffFrequency;
	BaseCutoffFrequency = CutoffFrequency;
	FilterQ = Settings.FilterQ;

	ModData.Reset();
	
	FAudioDeviceManager* AudioDeviceManager = FAudioDeviceManager::Get();
	if (AudioDeviceManager && Settings.AudioBusModulation.Num() > 0)
	{
		Audio::FMixerDevice* MixerDevice = (Audio::FMixerDevice*)AudioDeviceManager->GetAudioDeviceRaw(AudioDeviceId);
		if (MixerDevice)
		{
			for (FSourceEffectFilterAudioBusModulationSettings& BusModulationSettings : Settings.AudioBusModulation)
			{
				if (BusModulationSettings.AudioBus)
				{
					FAudioBusModulationData& NewModData = ModData.AddDefaulted_GetRef();

					uint32 AudioBusId = BusModulationSettings.AudioBus->GetUniqueID();
					UAudioBusSubsystem* AudioBusSubsystem = MixerDevice->GetSubsystem<UAudioBusSubsystem>();
					check(AudioBusSubsystem);
					NewModData.AudioBusPatch = AudioBusSubsystem->AddPatchOutputForAudioBus(Audio::FAudioBusKey(AudioBusId), MixerDevice->GetNumOutputFrames(), BusModulationSettings.AudioBus->GetNumChannels());

					NewModData.MinFreqModValue = BusModulationSettings.MinFrequencyModulation;
					NewModData.MaxFreqModValue = BusModulationSettings.MaxFrequencyModulation;
					NewModData.MinResModValue = BusModulationSettings.MinResonanceModulation;
					NewModData.MaxResModValue = BusModulationSettings.MaxResonanceModulation;
					NewModData.EnvelopeGain = BusModulationSettings.EnvelopeGainMultiplier;

					Audio::FEnvelopeFollowerInitParams EnvelopeFollowerInitParams;
					EnvelopeFollowerInitParams.SampleRate = SampleRate;
					EnvelopeFollowerInitParams.NumChannels = 1;
					EnvelopeFollowerInitParams.AttackTimeMsec = BusModulationSettings.EnvelopeFollowerAttackTimeMsec;
					EnvelopeFollowerInitParams.ReleaseTimeMsec = BusModulationSettings.EnvelopeFollowerReleaseTimeMsec;

					NewModData.AudioBusEnvelopeFollower.Init(EnvelopeFollowerInitParams);
				}
			}
		}
	}

	UpdateFilter();
}

void FSourceEffectFilter::ProcessAudio(const FSoundEffectSourceInputData& InData, float* OutAudioBufferData)
{
	float ModFrequency = 0.0f;
	float ModQ = 0.0f;

	for (FAudioBusModulationData& Mod : ModData)
	{
		if (!Mod.AudioBusPatch)
		{
			continue;
		}

		ScratchModBuffer.Reset();
		ScratchModBuffer.AddZeroed(InData.NumSamples * NumChannels);

		Mod.AudioBusPatch->PopAudio(ScratchModBuffer.GetData(), ScratchModBuffer.Num(), true);

		float EnvelopeValue = 0.0f;

		// If our modulation buffer (and this source effect) are both 2 channels, we need to downmix to mono before envelope following the signal
		if (NumChannels == 2)
		{
			ScratchEnvFollowerBuffer.Reset();
			ScratchEnvFollowerBuffer.AddUninitialized(InData.NumSamples);

			Audio::BufferSum2ChannelToMonoFast(ScratchModBuffer, ScratchEnvFollowerBuffer);
			Audio::ArrayMultiplyByConstantInPlace(ScratchEnvFollowerBuffer, 0.5f);

			Mod.AudioBusEnvelopeFollower.ProcessAudio(ScratchEnvFollowerBuffer.GetData(), InData.NumSamples);
		}
		else if (NumChannels == 1)
		{
			Mod.AudioBusEnvelopeFollower.ProcessAudio(ScratchModBuffer.GetData(), InData.NumSamples);
		}
		else
		{
			checkNoEntry();
		}

		const TArray<float>& CurrentEnvelopeValues = Mod.AudioBusEnvelopeFollower.GetEnvelopeValues();
		if (ensure(CurrentEnvelopeValues.Num() == 1))
		{
			EnvelopeValue = FMath::Clamp(CurrentEnvelopeValues[0], 0.f, 1.f);
		}
		else
		{
			EnvelopeValue = 0.f;
		}


		if (Mod.FilterParam == ESourceEffectFilterParam::FilterFrequency)
		{
			ModFrequency += FMath::Lerp(Mod.MinFreqModValue, Mod.MaxFreqModValue, FMath::Clamp(EnvelopeValue * Mod.EnvelopeGain, 0.0f, 1.0f));;
		}
		else
		{
			ModQ += FMath::Lerp(Mod.MinResModValue, Mod.MaxResModValue, FMath::Clamp(EnvelopeValue * Mod.EnvelopeGain, 0.0f, 1.0f));;
		}

		CutoffFrequency = BaseCutoffFrequency * Audio::GetFrequencyMultiplier(ModFrequency);
		FilterQ = FMath::Clamp(BaseFilterQ + ModQ, 0.1f, 10.0f);


		// Update the filter parameters
		UpdateFilter();
	}

	CurrentFilter->ProcessAudio(InData.InputSourceEffectBufferPtr, InData.NumSamples, OutAudioBufferData);
}

void USourceEffectFilterPreset::SetSettings(const FSourceEffectFilterSettings& InSettings)
{
	UpdateSettings(InSettings);
}
