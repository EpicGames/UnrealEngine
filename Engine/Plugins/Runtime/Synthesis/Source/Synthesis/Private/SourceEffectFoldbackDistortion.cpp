// Copyright Epic Games, Inc. All Rights Reserved.

#include "SourceEffects/SourceEffectFoldbackDistortion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SourceEffectFoldbackDistortion)

void FSourceEffectFoldbackDistortion::Init(const FSoundEffectSourceInitData& InitData)
{
	bIsActive = true;
	FoldbackDistortion.Init(InitData.SampleRate, InitData.NumSourceChannels);
}

void FSourceEffectFoldbackDistortion::OnPresetChanged()
{
	GET_EFFECT_SETTINGS(SourceEffectFoldbackDistortion);

	FoldbackDistortion.SetInputGainDb(Settings.InputGainDb);
	FoldbackDistortion.SetThresholdDb(Settings.ThresholdDb);
	FoldbackDistortion.SetOutputGainDb(Settings.OutputGainDb);
}

void FSourceEffectFoldbackDistortion::ProcessAudio(const FSoundEffectSourceInputData& InData, float* OutAudioBufferData)
{
	FoldbackDistortion.ProcessAudio(InData.InputSourceEffectBufferPtr, InData.NumSamples, OutAudioBufferData);
}

void USourceEffectFoldbackDistortionPreset::SetSettings(const FSourceEffectFoldbackDistortionSettings& InSettings)
{
	UpdateSettings(InSettings);
}
