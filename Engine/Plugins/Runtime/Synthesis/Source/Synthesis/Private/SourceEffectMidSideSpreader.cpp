// Copyright Epic Games, Inc. All Rights Reserved.

#include "SourceEffects/SourceEffectMidSideSpreader.h"
#include "DSP/Dsp.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SourceEffectMidSideSpreader)

FSourceEffectMidSideSpreader::FSourceEffectMidSideSpreader()
	: MidScale(1.0f)
	, SideScale(1.0f)
	, NumChannels(0)
{}

void FSourceEffectMidSideSpreader::Init(const FSoundEffectSourceInitData& InInitData)
{
	// Data initialization
	bIsActive = true;
	NumChannels = InInitData.NumSourceChannels;
}

void FSourceEffectMidSideSpreader::OnPresetChanged()
{
	// Macro to retrieve the current settings value of the parent preset asset.
	GET_EFFECT_SETTINGS(SourceEffectMidSideSpreader);

	// Update the instance's variables based on the settings values. 
	// Note that Settings variable was created by the GET_EFFECT_SETTINGS macro.
	SpreaderSettings = Settings;

	// Convert to radians between 0.0 and PI/2
	float SpreadScale = Settings.SpreadAmount * 0.5f * PI;

	// Compute equal power relationship between Mid and Side
	FMath::SinCos(&SideScale, &MidScale, SpreadScale);

	// Adjust gain so 0.5f Spread results in a 1.0f to 1.0f gain ratio between Mid and Side
	MidScale *= 1.414f;
	SideScale *= 1.414f;

	// Clamp values if not Equal Power
	if (!SpreaderSettings.bEqualPower)
	{
		MidScale = FMath::Clamp(MidScale, 0.0f, 1.0f);
		SideScale = FMath::Clamp(SideScale, 0.0f, 1.0f);
	}
}

void FSourceEffectMidSideSpreader::ProcessAudio(const FSoundEffectSourceInputData& InData, float* OutAudioBufferData)
{
	// Copy our Input Buffer to our Output Buffer
	FMemory::Memcpy(OutAudioBufferData, InData.InputSourceEffectBufferPtr, sizeof(float)*InData.NumSamples);

	// We only want to process stereo sources
	if (NumChannels == 2)
	{	
		// If our input is LR, we need to encode it to MS
		if (SpreaderSettings.InputMode == EStereoChannelMode::LeftRight)
		{
			for (int32 SampleIndex = 0; SampleIndex < InData.NumSamples; SampleIndex += NumChannels)
			{
				Audio::EncodeMidSide(OutAudioBufferData[SampleIndex], OutAudioBufferData[SampleIndex + 1]);

				OutAudioBufferData[SampleIndex] *= MidScale;
				OutAudioBufferData[SampleIndex + 1] *= SideScale;
			}
		}
		else
		{
			// We should now be in MS mode, now we can apply our gain scalars
			for (int32 SampleIndex = 0; SampleIndex < InData.NumSamples; SampleIndex += NumChannels)
			{
				OutAudioBufferData[SampleIndex] *= MidScale;
				OutAudioBufferData[SampleIndex + 1] *= SideScale;
			}
		}

		// If our output is LR we need to decode from MS
		if (SpreaderSettings.OutputMode == EStereoChannelMode::LeftRight)
		{
			for (int32 SampleIndex = 0; SampleIndex < InData.NumSamples; SampleIndex += NumChannels)
			{
				Audio::DecodeMidSide(OutAudioBufferData[SampleIndex], OutAudioBufferData[SampleIndex + 1]);
			}
		}
	}
}

void USourceEffectMidSideSpreaderPreset::SetSettings(const FSourceEffectMidSideSpreaderSettings& InSettings)
{
	// Performs necessary broadcast to effect instances
	UpdateSettings(InSettings);
}
