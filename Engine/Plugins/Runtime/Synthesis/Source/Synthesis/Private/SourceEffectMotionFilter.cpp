// Copyright Epic Games, Inc. All Rights Reserved.

#include "SourceEffects/SourceEffectMotionFilter.h"

#include "DSP/FloatArrayMath.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SourceEffectMotionFilter)

FSourceEffectMotionFilter::FSourceEffectMotionFilter()
	: Topology(ESourceEffectMotionFilterTopology::ParallelMode)
	, FilterMixAmount(0.0f)
	, DryVolumeScalar(1.0f)
	, ModSourceTimeStamp(0.0)
	, LastDistance(0.0f)
	, LastEmitterWorldPosition(0.0f, 0.0f, 0.0f)
	, LastListenerWorldPosition(0.0f, 0.0f, 0.0f)
	, LastEmitterNormalizedPosition(0.0f, 0.0f, 0.0f)
	, FilterAMixScale(0.0f)
	, FilterBMixScale(0.0f)
	, FilterAOutputScale(1.0f)
	, FilterBOutputScale(1.0f)
	, SampleRate(48000.0f)
	, NumChannels(2)
	, ChannelRate(96000.0f)
{
}
	
void FSourceEffectMotionFilter::Init(const FSoundEffectSourceInitData& InInitData)
{
	// Use InitData to setup DSP effects that depend on sample rate, etc.
	bIsActive = true;
	Topology = ESourceEffectMotionFilterTopology::ParallelMode;

	// Init Filter Mix
	FilterMixAmount = 0.0f;

	// Reserve space for as many key/value pairs as needed for all the key values
	ModMap.Empty((uint8)ESourceEffectMotionFilterModDestination::Count);
	ModMapOutputRange.Empty((uint8)ESourceEffectMotionFilterModDestination::Count);

	// Init size of Mod Matrix
	ModMatrix.SetNumZeroed((uint8)ESourceEffectMotionFilterModSource::Count);
	for (int32 i = 0; i < ModMatrix.Num(); i++)
	{
		ModMatrix[i].SetNumZeroed((uint8)ESourceEffectMotionFilterModDestination::Count);
	}

	// Init size of Target Matrix
	TargetMatrix.SetNumZeroed((uint8)ESourceEffectMotionFilterModSource::Count);
	for (int32 i = 0; i < TargetMatrix.Num(); i++)
	{
		TargetMatrix[i].SetNumZeroed((uint8)ESourceEffectMotionFilterModDestination::Count);
	}

	// Init size of Last Target Matrix
	LastTargetMatrix.SetNumZeroed((uint8)ESourceEffectMotionFilterModSource::Count);
	for (int32 i = 0; i < LastTargetMatrix.Num(); i++)
	{
		LastTargetMatrix[i].SetNumZeroed((uint8)ESourceEffectMotionFilterModDestination::Count);
	}

	// Init size of Linear Ease Matrix
	LinearEaseMatrix.SetNumZeroed((uint8)ESourceEffectMotionFilterModSource::Count);
	for (int32 i = 0; i < LinearEaseMatrix.Num(); i++)
	{
		LinearEaseMatrix[i].SetNumZeroed((uint8)ESourceEffectMotionFilterModDestination::Count);
	}

	// Initialize Linear Ease Init State bool
	LinearEaseMatrixInit.SetNumZeroed((uint8)ESourceEffectMotionFilterModSource::Count);
	for (int32 i = 0; i < LinearEaseMatrixInit.Num(); i++)
	{
		LinearEaseMatrixInit[i].SetNumZeroed((uint8)ESourceEffectMotionFilterModDestination::Count);
	}

	DryVolumeScalar = 1.0f;

	// Init Mod Source Array
	ModSources.SetNumZeroed((uint8)ESourceEffectMotionFilterModSource::Count);

	// Set Initial Clock Stamp
	ModSourceTimeStamp = InInitData.AudioClock;

	// Init default params
	LastDistance = 0.0f;
	LastEmitterWorldPosition.Set(0.0f, 0.0f, 0.0f);
	LastListenerWorldPosition.Set(0.0f, 0.0f, 0.0f);

	// Init Mod Destination Arrays
	BaseDestinationValues.SetNumZeroed((uint8)ESourceEffectMotionFilterModDestination::Count);
	ModDestinationValues.SetNumZeroed((uint8)ESourceEffectMotionFilterModDestination::Count);
	ModDestinationUpdateTimeMS.SetNumZeroed((uint8)ESourceEffectMotionFilterModDestination::Count);

	// Init Output Scalars
	FilterAMixScale = 0.0f;
	FilterBMixScale = 0.0f;
	FilterAOutputScale = 1.0f;
	FilterBOutputScale = 1.0f;

	// Pass Init Data to relevant member variables
	SampleRate = InInitData.SampleRate;
	NumChannels = InInitData.NumSourceChannels;

	// Check for bad Init Data
	check(SampleRate > 0);
	check(NumChannels > 0);

	ChannelRate = SampleRate * NumChannels;

	// Initialize Filters
	MotionFilterA.StateVarFilter.Init(SampleRate, NumChannels);
	MotionFilterA.LadderFilter.Init(SampleRate, NumChannels);
	MotionFilterA.OnePoleFilter.Init(SampleRate, NumChannels);
	MotionFilterB.StateVarFilter.Init(SampleRate, NumChannels);
	MotionFilterB.LadderFilter.Init(SampleRate, NumChannels);
	MotionFilterB.OnePoleFilter.Init(SampleRate, NumChannels);
}

void FSourceEffectMotionFilter::OnPresetChanged()
{
	// Macro to retrieve the current settings value of the parent preset asset.
	GET_EFFECT_SETTINGS(SourceEffectMotionFilter);

	// Update the instance's variables based on the settings values. 
	// Note that Settings variable was created by the GET_EFFECT_SETTINGS macro.

	Topology = Settings.MotionFilterTopology;

	// Update Filter A Settings
	UpdateFilter(&MotionFilterA,
		Settings.FilterASettings.FilterCircuit,
		Settings.FilterASettings.FilterType,
		Settings.FilterASettings.CutoffFrequency,
		Settings.FilterASettings.FilterQ);

	// Update Filter B Settings
	UpdateFilter(&MotionFilterB,
		Settings.FilterBSettings.FilterCircuit,
		Settings.FilterBSettings.FilterType,
		Settings.FilterBSettings.CutoffFrequency,
		Settings.FilterBSettings.FilterQ);

	FilterMixAmount = Settings.MotionFilterMix;
	FilterAOutputScale = 1.0f;
	FilterBOutputScale = 1.0f;

	// Collect Modulation Map
	TArray<ESourceEffectMotionFilterModDestination> ModKeys;
	ModMap.Empty((uint8)ESourceEffectMotionFilterModDestination::Count);
	ModMapOutputRange.Empty((uint8)ESourceEffectMotionFilterModDestination::Count);
	Settings.ModulationMappings.GenerateKeyArray(ModKeys);

	for (const ESourceEffectMotionFilterModDestination& Key : ModKeys)
	{
		if (Settings.ModulationMappings.Find(Key))
		{
			FSourceEffectMotionFilterModulationSettings Value = *Settings.ModulationMappings.Find(Key);
			ModMap.Add(Key, Value);

			// Set random offset values used for instanced variation--reset every time the Set Settings is called
			FVector2D ActualValueRange;
			ActualValueRange.X = FMath::FRandRange(Value.ModulationOutputMinimumRange.X, Value.ModulationOutputMinimumRange.Y);
			ActualValueRange.Y = FMath::FRandRange(Value.ModulationOutputMaximumRange.X, Value.ModulationOutputMaximumRange.Y);

			ModMapOutputRange.Add(Key, ActualValueRange);
		}
	}

	// Reset Mod Matrix
	for (int32 i = 0; i < ModMatrix.Num(); i++)
	{
		ModMatrix[i].SetNumZeroed((uint8)ESourceEffectMotionFilterModDestination::Count, EAllowShrinking::Yes);
	}

	for (int32 i = 0; i < LinearEaseMatrixInit.Num(); i++)
	{
		for (int32 j = 0; j < LinearEaseMatrixInit[i].Num(); j++)
		{
			LinearEaseMatrixInit[i][j] = false;
		}
	}

	DryVolumeScalar = FMath::Clamp(Audio::ConvertToLinear(Settings.DryVolumeDb), 0.0f, 3.0f);

	// Cache Base Destination Values
	BaseDestinationValues[(uint8)ESourceEffectMotionFilterModDestination::FilterACutoffFrequency] = FMath::Clamp(Settings.FilterASettings.CutoffFrequency, 20.0f, 15000.0f);
	BaseDestinationValues[(uint8)ESourceEffectMotionFilterModDestination::FilterAResonance] = FMath::Clamp(Settings.FilterASettings.FilterQ, 0.5f, 10.0f);
	BaseDestinationValues[(uint8)ESourceEffectMotionFilterModDestination::FilterBCutoffFrequency] = FMath::Clamp(Settings.FilterBSettings.CutoffFrequency, 20.0f, 15000.0f);
	BaseDestinationValues[(uint8)ESourceEffectMotionFilterModDestination::FilterBResonance] = FMath::Clamp(Settings.FilterBSettings.FilterQ, 0.5f, 10.0f);
	BaseDestinationValues[(uint8)ESourceEffectMotionFilterModDestination::FilterMix] = FMath::Clamp(Settings.MotionFilterMix, -1.0f, 1.0f);
}

void FSourceEffectMotionFilter::ProcessAudio(const FSoundEffectSourceInputData& InData, float* OutAudioBufferData)
{
	// First update internal tracking of Source/Listener data
	UpdateModulationSources(InData);

	// Check that ChannelRate is greater than 0 before dividing by ChannelRate
	check(ChannelRate > 0);

	// Check and update if we have actual modulation
	if (UpdateModulationMatrix(InData.NumSamples / ChannelRate))
	{
		// Update Destination Values if we successfully updated the Mod Matrix
		UpdateModulationDestinations();

		// Apply Mod Destination Values
		ApplyFilterModulation(ModDestinationValues);
	}
	else
	{
		// There is no modulation, so apply base destination values
		ApplyFilterModulation(BaseDestinationValues);
	}

	// We've now updated our Filters, we can now set up our Filter Processing
	if (InData.NumSamples != ScratchBufferA.Num())
	{
		ScratchBufferA.Reset();
		ScratchBufferA.AddUninitialized(InData.NumSamples);
		ScratchBufferB.Reset();
		ScratchBufferB.AddUninitialized(InData.NumSamples);
	}

	// Set up Mix Blend Scalar Values in preparation for Filter Processing Chain
	float MixScale = (FilterMixAmount + 1.0f) * 0.25f * PI;
	FMath::SinCos(&FilterBMixScale, &FilterAMixScale, MixScale);

	FilterAMixScale *= FilterAOutputScale;
	FilterBMixScale *= FilterBOutputScale;

	FilterAMixScale = FMath::Clamp(FilterAMixScale, 0.0f, 2.0f);
	FilterBMixScale = FMath::Clamp(FilterBMixScale, 0.0f, 2.0f);

	TArrayView<float> ScratchBufferAView(ScratchBufferA.GetData(), InData.NumSamples);
	TArrayView<float> ScratchBufferBView(ScratchBufferB.GetData(), InData.NumSamples);
	TArrayView<float> OutAudioBufferDataView(OutAudioBufferData, InData.NumSamples);
	TArrayView<float> InputSourceEffectBufferView(InData.InputSourceEffectBufferPtr, InData.NumSamples);

	// Mix and Processing chain depends on Topology
	if (Topology == ESourceEffectMotionFilterTopology::ParallelMode)
	{
		// Parallel Topology means Filter A and Filter B process input separately then are combined at the end based on the Filter Mix
		MotionFilterA.CurrentFilter->ProcessAudio(InData.InputSourceEffectBufferPtr, InData.NumSamples, ScratchBufferA.GetData());
		MotionFilterB.CurrentFilter->ProcessAudio(InData.InputSourceEffectBufferPtr, InData.NumSamples, ScratchBufferB.GetData());

		// Mix Dry + Filter A + Filter B
		Audio::ArrayMixIn(InputSourceEffectBufferView, OutAudioBufferDataView, DryVolumeScalar);
		Audio::ArrayMixIn(ScratchBufferAView, OutAudioBufferDataView, FilterAMixScale);
		Audio::ArrayMixIn(ScratchBufferBView, OutAudioBufferDataView, FilterBMixScale);

	}
	else if (Topology == ESourceEffectMotionFilterTopology::SerialMode)
	{
		// Serial Topology means Filter A processes first with a Dry blend based on Mix, then Filter B processes that output with its own non-processed blend with Mix
		MotionFilterA.CurrentFilter->ProcessAudio(InData.InputSourceEffectBufferPtr, InData.NumSamples, ScratchBufferA.GetData());

		// Mix Dry + Filter A
		Audio::ArrayMultiplyByConstantInPlace(ScratchBufferAView, FilterAMixScale);
		Audio::ArrayMixIn(InputSourceEffectBufferView, ScratchBufferAView, DryVolumeScalar);

		// Process Filter A Wet/Dry Mix into Filter B
		MotionFilterB.CurrentFilter->ProcessAudio(ScratchBufferA.GetData(), InData.NumSamples, ScratchBufferB.GetData());

		// Mix Dry + Filter B Output
		Audio::ArrayMixIn(InputSourceEffectBufferView, OutAudioBufferDataView, DryVolumeScalar);
		Audio::ArrayMixIn(ScratchBufferBView, OutAudioBufferDataView, FilterBMixScale);
	}
	else
	{
		// No other condition was met, pass-through audio.
		// Copy our Input Buffer to our Output Buffer
		FMemory::Memcpy(OutAudioBufferData, InData.InputSourceEffectBufferPtr, sizeof(float)*InData.NumSamples);
	}
}

void FSourceEffectMotionFilter::UpdateFilter(FMotionFilter* MotionFilter, ESourceEffectMotionFilterCircuit FilterCircuitType, ESourceEffectMotionFilterType MotionFilterType, float FilterFrequency, float FilterQ)
{
	Audio::IFilter* CurrentFilter = nullptr;

	check(MotionFilter != nullptr);

	switch (FilterCircuitType)
	{
	case ESourceEffectMotionFilterCircuit::OnePole:
	{
		CurrentFilter = &MotionFilter->OnePoleFilter;
	}
	break;

	case ESourceEffectMotionFilterCircuit::StateVariable:
	{
		CurrentFilter = &MotionFilter->StateVarFilter;
	}
	break;

	case ESourceEffectMotionFilterCircuit::Ladder:
	{
		CurrentFilter = &MotionFilter->LadderFilter;
	}
	break;
	}

	CurrentFilter->SetFilterType((Audio::EFilter::Type)(uint8)MotionFilterType);
	CurrentFilter->SetFrequency(FilterFrequency);
	CurrentFilter->SetQ(FilterQ);
	CurrentFilter->Update();
	MotionFilter->CurrentFilter = CurrentFilter;
	MotionFilter->CurrentFilterCircuit = FilterCircuitType;
	MotionFilter->FilterType = (Audio::EFilter::Type)(uint8)MotionFilterType;
	MotionFilter->FilterFrequency = FMath::Clamp(FilterFrequency, 20.0f, 15000.0f);
	MotionFilter->FilterQ = FMath::Clamp(FilterQ, 0.5f, 10.0f);
}

void FSourceEffectMotionFilter::ApplyFilterModulation(const TArray<float>& DestinationSettings)
{
	// Update Filter A Settings
	UpdateFilter(&MotionFilterA,
		MotionFilterA.CurrentFilterCircuit,
		(ESourceEffectMotionFilterType)(uint8)MotionFilterA.FilterType,
		DestinationSettings[(uint8)ESourceEffectMotionFilterModDestination::FilterACutoffFrequency],
		DestinationSettings[(uint8)ESourceEffectMotionFilterModDestination::FilterAResonance]);

	// Update Filter B Settings
	UpdateFilter(&MotionFilterB,
		MotionFilterB.CurrentFilterCircuit,
		(ESourceEffectMotionFilterType)(uint8)MotionFilterB.FilterType,
		DestinationSettings[(uint8)ESourceEffectMotionFilterModDestination::FilterBCutoffFrequency],
		DestinationSettings[(uint8)ESourceEffectMotionFilterModDestination::FilterBResonance]);

	FilterMixAmount = DestinationSettings[(uint8)ESourceEffectMotionFilterModDestination::FilterMix];
	FilterAOutputScale = Audio::ConvertToLinear(ModDestinationValues[(uint8)ESourceEffectMotionFilterModDestination::FilterAOutputVolumeDB]);
	FilterBOutputScale = Audio::ConvertToLinear(ModDestinationValues[(uint8)ESourceEffectMotionFilterModDestination::FilterBOutputVolumeDB]);

}


void FSourceEffectMotionFilter::UpdateModulationSources(const FSoundEffectSourceInputData& InData)
{
	if (!FMath::IsNearlyEqual(InData.SpatParams.AudioClock, ModSourceTimeStamp, 0.001))
	{
		// Get time delta from last update
		float TimeDelta = (float)(InData.SpatParams.AudioClock - ModSourceTimeStamp);

		// TimeDelta must not be 0.0f
		check(TimeDelta != 0.0f);

		// Update the mod source array
		// Distance From Listener
		ModSources[(uint8)ESourceEffectMotionFilterModSource::DistanceFromListener] = InData.SpatParams.Distance;

		// Speed Relative to Listener
		ModSources[(uint8)ESourceEffectMotionFilterModSource::SpeedRelativeToListener] = FMath::Abs(InData.SpatParams.Distance - LastDistance) / TimeDelta;

		// Speed of Emitter
		FVector EmitterPositionDelta = InData.SpatParams.EmitterWorldPosition - LastEmitterWorldPosition;
		ModSources[(uint8)ESourceEffectMotionFilterModSource::SpeedOfSourceEmitter] = EmitterPositionDelta.Size() / TimeDelta;

		// Speed of Listener
		FVector ListenerWorldPosition = InData.SpatParams.ListenerPosition;
		FVector ListenerWorldPositionDelta = ListenerWorldPosition - LastListenerWorldPosition;

		ModSources[(uint8)ESourceEffectMotionFilterModSource::SpeedOfListener] = ListenerWorldPositionDelta.Size() / TimeDelta;

		// Angle Delta of Emitter from the perspective of the Listener
		FVector CurrentEmitterPosition = InData.SpatParams.EmitterPosition;
		float EmitterDotProduct = FVector::DotProduct(CurrentEmitterPosition, LastEmitterNormalizedPosition);
		float ListenerEmitterAngle = FMath::Acos(EmitterDotProduct) * 180 / PI;
		ModSources[(uint8)ESourceEffectMotionFilterModSource::SpeedOfAngleDelta] = FMath::Abs(ListenerEmitterAngle) / TimeDelta;


		// Update cached distances and positions
		LastDistance = InData.SpatParams.Distance;
		LastEmitterWorldPosition = InData.SpatParams.EmitterWorldPosition;
		LastListenerWorldPosition = ListenerWorldPosition;
		LastEmitterNormalizedPosition = InData.SpatParams.EmitterPosition;

		// Update TimeStamp
		ModSourceTimeStamp = InData.SpatParams.AudioClock;
	}
}

bool FSourceEffectMotionFilter::UpdateModulationMatrix(const float UpdateTime)
{
	if (ModMap.Num())
	{
		// Get update time in ms
		// const float UpdateTimeInMS = UpdateTime * 1000.0f;

		// Feed the Matrix
		TArray<ESourceEffectMotionFilterModDestination> ModKeys;
		ModMap.GenerateKeyArray(ModKeys);

		// This loop fills up the Matrix with mapped output values
		for (const ESourceEffectMotionFilterModDestination& Destination : ModKeys)
		{
			// Get Source and Values to access mod matrix values
			const FSourceEffectMotionFilterModulationSettings& Value = ModMap.FindRef(Destination);
			ESourceEffectMotionFilterModSource Source = Value.ModulationSource;

			FVector2D OutputRange = *ModMapOutputRange.Find(Destination);

			float& ModMatrixIndex = ModMatrix[(uint8)Source][(uint8)Destination];
			float& TargetMatrixIndex = TargetMatrix[(uint8)Source][(uint8)Destination];
			float& LastTargetMatrixIndex = LastTargetMatrix[(uint8)Source][(uint8)Destination];
			Audio::FLinearEase& LinearEaseMatrixIndex = LinearEaseMatrix[(uint8)Source][(uint8)Destination];
				
			// Check if this Linear Ease needs to be reinitialized
			if (!LinearEaseMatrixInit[(uint8)Source][(uint8)Destination])
			{
				// UpdateTime must not be 0.0f
				check(UpdateTime != 0.0f);

				LinearEaseMatrixIndex.Init(1.0f / UpdateTime);

				LinearEaseMatrixInit[(uint8)Source][(uint8)Destination] = true;
			}

			// Update Target Matrix value
			TargetMatrixIndex = FMath::GetMappedRangeValueClamped(Value.ModulationInputRange, OutputRange, FVector2D::FReal(ModSources[(uint8)Source]));

			// If Target Matrix value has been changed significantly, then we need to re-target our Linear Ease
			if (!FMath::IsNearlyEqual(LastTargetMatrixIndex, TargetMatrixIndex, 0.001f))
			{
				LinearEaseMatrixIndex.SetValue(TargetMatrixIndex, (Value.UpdateEaseMS / 1000.0f));
			}

			// Update the Mod Matrix
 			ModMatrixIndex = LinearEaseMatrixIndex.GetNextValue(1);

			// Cache last Target Value
			LastTargetMatrixIndex = TargetMatrixIndex;
		}

		return true;
	}

	return false;
}

void FSourceEffectMotionFilter::UpdateModulationDestinations()
{
	// Clear Mod Destination Values
	for (uint8 i = 0; i < (uint8)ESourceEffectMotionFilterModDestination::Count; i++)
	{
		ModDestinationValues[i] = 0.0f;
	}

	// Crawl Mod Matrix
	for (uint8 i = 0; i < (uint8)ESourceEffectMotionFilterModDestination::Count; i++)
	{
		// Check if Key for modulation exists
		if (ModMap.Contains((ESourceEffectMotionFilterModDestination)i))
		{
			// Loop through each Source feeding the Destination value and get the max
			for (uint8 j = 0; j < (uint8)ESourceEffectMotionFilterModSource::Count; j++)
			{
				if (FMath::Abs(ModDestinationValues[i]) > FMath::Abs(ModMatrix[j][i]))
				{
					ModDestinationValues[i] = FMath::Sign(ModDestinationValues[i]) * FMath::Max(FMath::Abs(ModDestinationValues[i]), FMath::Abs(ModMatrix[j][i]));
				}
				else
				{
					ModDestinationValues[i] = FMath::Sign(ModMatrix[j][i]) * FMath::Max(FMath::Abs(ModDestinationValues[i]), FMath::Abs(ModMatrix[j][i]));
				}
			}
		}
		else
		{
			// Else pass in Base value
			ModDestinationValues[i] = BaseDestinationValues[i];
		}
	}
}

void USourceEffectMotionFilterPreset::SetSettings(const FSourceEffectMotionFilterSettings& InSettings)
{
	// Performs necessary broadcast to effect instances
	UpdateSettings(InSettings);
}
