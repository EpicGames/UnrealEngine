// Copyright Epic Games, Inc. All Rights Reserved.

#include "SourceEffects/SourceEffectPanner.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SourceEffectPanner)

void FSourceEffectPanner::Init(const FSoundEffectSourceInitData& InitData)
{
	bIsActive = true;
	NumChannels = InitData.NumSourceChannels;
}

void FSourceEffectPanner::OnPresetChanged()
{
	GET_EFFECT_SETTINGS(SourceEffectPanner);

	// Normalize the spread value to be between 0.0 and 1.0
	float SpreadValue = 0.5f * (1.0f + Settings.Spread);

	// Convert to radians between 0.0 and PI/2
	SpreadValue *= 0.5f * PI;

	// Normalize the panning value to be between 0.0 and 1.0
	float PanValue = 0.5f * (1.0f - Settings.Pan);

	// Convert to radians between 0.0 and PI/2
	PanValue *= 0.5f * PI;

	// Use the "cosine" equal power panning law to compute the spread gain amounts
	FMath::SinCos(&SpreadGains[0], &SpreadGains[1], SpreadValue);

	// Use the "cosine" equal power panning law to compute a smooth pan based off our parameter
	FMath::SinCos(&PanGains[0], &PanGains[1], PanValue);

	// Clamp this to be between 0.0 and 1.0 since SinCos is fast and may have values greater than 1.0 or less than 0.0
	for (int32 i = 0; i < 2; ++i)
	{
		SpreadGains[i] = FMath::Clamp(SpreadGains[i], 0.0f, 1.0f);
		PanGains[i] = FMath::Clamp(PanGains[i], 0.0f, 1.0f);
	}

}

void FSourceEffectPanner::ProcessAudio(const FSoundEffectSourceInputData& InData, float* OutAudioBufferData)
{
	if (NumChannels != 2)
	{
		FMemory::Memcpy(OutAudioBufferData, InData.InputSourceEffectBufferPtr, sizeof(float)*InData.NumSamples);
	}
	else
	{
		for (int32 SampleIndex = 0; SampleIndex < InData.NumSamples; SampleIndex += NumChannels)
		{
			const float LeftChannel = InData.InputSourceEffectBufferPtr[SampleIndex];
			const float RightChannel = InData.InputSourceEffectBufferPtr[SampleIndex + 1];

			// Left channel spread calculation:
			// Left channel mix = Sin of Left + Cos of Right
			// Right channel spread calculation:
			// Right channel mix = Cos of Left + Sin of Right
			// Then scale the pan value output with the channel inputs. 
			OutAudioBufferData[SampleIndex] = PanGains[0] * ((SpreadGains[0] * LeftChannel) + (SpreadGains[1] * RightChannel));
			OutAudioBufferData[SampleIndex + 1] = PanGains[1] * ((SpreadGains[1] * LeftChannel) + (SpreadGains[0] * RightChannel));
		}
	}
}

void USourceEffectPannerPreset::SetSettings(const FSourceEffectPannerSettings& InSettings)
{
	UpdateSettings(InSettings);
}
