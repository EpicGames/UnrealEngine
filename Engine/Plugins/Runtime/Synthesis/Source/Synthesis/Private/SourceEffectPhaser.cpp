// Copyright Epic Games, Inc. All Rights Reserved.

#include "SourceEffects/SourceEffectPhaser.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SourceEffectPhaser)

void FSourceEffectPhaser::Init(const FSoundEffectSourceInitData& InitData)
{
	bIsActive = true;

	Phaser.Init(InitData.SampleRate, InitData.NumSourceChannels);
}

void FSourceEffectPhaser::OnPresetChanged()
{
	GET_EFFECT_SETTINGS(SourceEffectPhaser);

	Phaser.SetFrequency(Settings.Frequency);
	Phaser.SetWetLevel(Settings.WetLevel);
	Phaser.SetQuadPhase(Settings.UseQuadraturePhase);
	Phaser.SetFeedback(Settings.Feedback);

	Phaser.SetLFOType((Audio::ELFO::Type)Settings.LFOType);
}

void FSourceEffectPhaser::ProcessAudio(const FSoundEffectSourceInputData& InData, float* OutAudioBufferData)
{
	Phaser.ProcessAudio(InData.InputSourceEffectBufferPtr, InData.NumSamples, OutAudioBufferData);
}

void USourceEffectPhaserPreset::SetSettings(const FSourceEffectPhaserSettings& InSettings)
{
	UpdateSettings(InSettings);
}
