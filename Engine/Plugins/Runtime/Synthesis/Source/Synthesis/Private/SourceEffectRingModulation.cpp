// Copyright Epic Games, Inc. All Rights Reserved.

#include "SourceEffects/SourceEffectRingModulation.h"
#include "AudioBusSubsystem.h"
#include "AudioMixerDevice.h"
#include "Sound/AudioBus.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SourceEffectRingModulation)

void FSourceEffectRingModulation::Init(const FSoundEffectSourceInitData& InitData)
{
	bIsActive = true;
	NumChannels = InitData.NumSourceChannels;
	RingModulation.Init(InitData.SampleRate, NumChannels);
	AudioDeviceId = InitData.AudioDeviceId;
}

void FSourceEffectRingModulation::OnPresetChanged()
{
	GET_EFFECT_SETTINGS(SourceEffectRingModulation);

	switch (Settings.ModulatorType)
	{
		default:
		case ERingModulatorTypeSourceEffect::Sine:
			RingModulation.SetModulatorWaveType(Audio::EOsc::Sine);
			break;

		case ERingModulatorTypeSourceEffect::Saw:
			RingModulation.SetModulatorWaveType(Audio::EOsc::Saw);
			break;

		case ERingModulatorTypeSourceEffect::Triangle:
			RingModulation.SetModulatorWaveType(Audio::EOsc::Triangle);
			break;

		case ERingModulatorTypeSourceEffect::Square:
			RingModulation.SetModulatorWaveType(Audio::EOsc::Square);
			break;
	}

	RingModulation.SetModulationDepth(Settings.Depth);
	RingModulation.SetModulationFrequency(Settings.Frequency);
	RingModulation.SetDryLevel(Settings.DryLevel);
	RingModulation.SetWetLevel(Settings.WetLevel);

	// If we're modulating the ring modulator with an audio bus, lets set up that patch source from the audio bus
	FAudioDeviceManager* AudioDeviceManager = FAudioDeviceManager::Get();
	if (Settings.AudioBusModulator && AudioDeviceManager)
	{
		Audio::FMixerDevice* MixerDevice = (Audio::FMixerDevice*)AudioDeviceManager->GetAudioDeviceRaw(AudioDeviceId);
		uint32 AudioBusId = Settings.AudioBusModulator->GetUniqueID();
		UAudioBusSubsystem* AudioBusSubsystem = MixerDevice->GetSubsystem<UAudioBusSubsystem>();
		check(AudioBusSubsystem);
		Audio::FPatchOutputStrongPtr AudioBusPatchOutputPtr = AudioBusSubsystem->AddPatchOutputForAudioBus(Audio::FAudioBusKey(AudioBusId), MixerDevice->GetNumOutputFrames(), Settings.AudioBusModulator->GetNumChannels());
		RingModulation.SetExternalPatchSource(AudioBusPatchOutputPtr);
	}
	else
	{
		RingModulation.SetExternalPatchSource(nullptr);
	}
}

void FSourceEffectRingModulation::ProcessAudio(const FSoundEffectSourceInputData& InData, float* OutAudioBufferData)
{
	RingModulation.ProcessAudio(InData.InputSourceEffectBufferPtr, InData.NumSamples, OutAudioBufferData);
}

void USourceEffectRingModulationPreset::SetSettings(const FSourceEffectRingModulationSettings& InSettings)
{
	UpdateSettings(InSettings);
}

