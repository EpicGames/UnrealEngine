// Copyright Epic Games, Inc. All Rights Reserved.

#include "SourceEffects/SourceEffectSimpleDelay.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SourceEffectSimpleDelay)

void FSourceEffectSimpleDelay::Init(const FSoundEffectSourceInitData& InitData)
{
	bIsActive = true;
	bIsInit = true;

	FeedbackSamples.AddZeroed(InitData.NumSourceChannels);
	Delays.AddDefaulted(InitData.NumSourceChannels);

	for (int32 i = 0; i < InitData.NumSourceChannels; ++i)
	{
		Delays[i].Init(InitData.SampleRate, 2.0f);
	}
}

void FSourceEffectSimpleDelay::OnPresetChanged()
{
	GET_EFFECT_SETTINGS(SourceEffectSimpleDelay);

	SettingsCopy = Settings;

	// If we are manually setting the delay, lets set it now on the delay line
	if (!SettingsCopy.bDelayBasedOnDistance)
	{
		for (Audio::FDelay& Delay : Delays)
		{
			Delay.SetEasedDelayMsec(SettingsCopy.DelayAmount * 1000.0f, bIsInit);
		}
		bIsInit = false;
	}
}

void FSourceEffectSimpleDelay::ProcessAudio(const FSoundEffectSourceInputData& InData, float* OutAudioBufferData)
{
	if (SettingsCopy.bDelayBasedOnDistance)
	{
		const float DistanceMeters = SettingsCopy.bUseDistanceOverride ?
			InData.SpatParams.AttenuationDistance * 0.1f
			: InData.SpatParams.Distance * 0.01f;

		const float DelayAmountMsec = 1000.0f * DistanceMeters / SettingsCopy.SpeedOfSound;

		for (Audio::FDelay& Delay : Delays)
		{
			Delay.SetEasedDelayMsec(DelayAmountMsec, bIsInit);
		}
		bIsInit = false;
	}

	int32 NumChannels = Delays.Num();
	for (int32 SampleIndex = 0; SampleIndex < InData.NumSamples; SampleIndex += NumChannels)
	{
		for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ++ChannelIndex)
		{
			Audio::FDelay& Delay = Delays[ChannelIndex];

			const float DrySample = InData.InputSourceEffectBufferPtr[SampleIndex + ChannelIndex] * SettingsCopy.DryAmount;
			const float WetSample = SettingsCopy.WetAmount * Delay.ProcessAudioSample(InData.InputSourceEffectBufferPtr[SampleIndex + ChannelIndex] + FeedbackSamples[ChannelIndex] * SettingsCopy.Feedback);

			OutAudioBufferData[SampleIndex + ChannelIndex] = DrySample + WetSample;
			FeedbackSamples[ChannelIndex] = OutAudioBufferData[SampleIndex + ChannelIndex];
		}
	}

}

void USourceEffectSimpleDelayPreset::SetSettings(const FSourceEffectSimpleDelaySettings& InSettings)
{
	UpdateSettings(InSettings);
}
