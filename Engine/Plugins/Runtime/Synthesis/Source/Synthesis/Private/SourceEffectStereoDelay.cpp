// Copyright Epic Games, Inc. All Rights Reserved.

#include "SourceEffects/SourceEffectStereoDelay.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SourceEffectStereoDelay)

void FSourceEffectStereoDelay::Init(const FSoundEffectSourceInitData& InitData)
{
	bIsActive = true;
	DelayStereo.Init(InitData.SampleRate, InitData.NumSourceChannels);
}

void FSourceEffectStereoDelay::OnPresetChanged()
{
	GET_EFFECT_SETTINGS(SourceEffectStereoDelay);

	DelayStereo.SetDelayTimeMsec(Settings.DelayTimeMsec);
	DelayStereo.SetFeedback(Settings.Feedback);
	DelayStereo.SetWetLevel(Settings.WetLevel);
	DelayStereo.SetDelayRatio(Settings.DelayRatio);
	DelayStereo.SetMode((Audio::EStereoDelayMode::Type)Settings.DelayMode);
	DelayStereo.SetFilterEnabled(Settings.bFilterEnabled);
	DelayStereo.SetFilterSettings((Audio::EBiquadFilter::Type)Settings.FilterType, Settings.FilterFrequency, Settings.FilterQ);
}

void FSourceEffectStereoDelay::ProcessAudio(const FSoundEffectSourceInputData& InData, float* OutAudioBufferData)
{
	DelayStereo.ProcessAudio(InData.InputSourceEffectBufferPtr, InData.NumSamples, OutAudioBufferData);
}

void USourceEffectStereoDelayPreset::SetSettings(const FSourceEffectStereoDelaySettings& InSettings)
{
	UpdateSettings(InSettings);
}
