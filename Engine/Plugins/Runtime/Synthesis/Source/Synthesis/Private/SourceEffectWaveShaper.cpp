// Copyright Epic Games, Inc. All Rights Reserved.

#include "SourceEffects/SourceEffectWaveShaper.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SourceEffectWaveShaper)

void FSourceEffectWaveShaper::Init(const FSoundEffectSourceInitData& InitData)
{
	bIsActive = true;
	WaveShaper.Init(InitData.SampleRate);
	NumChannels = InitData.NumSourceChannels;
}

void FSourceEffectWaveShaper::OnPresetChanged()
{
	GET_EFFECT_SETTINGS(SourceEffectWaveShaper);

	WaveShaper.SetAmount(Settings.Amount);
	WaveShaper.SetOutputGainDb(Settings.OutputGainDb);
}

void FSourceEffectWaveShaper::ProcessAudio(const FSoundEffectSourceInputData& InData, float* OutAudioBufferData)
{
	for (int32 SampleIndex = 0; SampleIndex < InData.NumSamples; SampleIndex += NumChannels)
	{
		for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ++ChannelIndex)
		{
			WaveShaper.ProcessAudio(InData.InputSourceEffectBufferPtr[SampleIndex + ChannelIndex], OutAudioBufferData[SampleIndex + ChannelIndex]);
		}
	}
}

void USourceEffectWaveShaperPreset::SetSettings(const FSourceEffectWaveShaperSettings& InSettings)
{
	UpdateSettings(InSettings);
}
