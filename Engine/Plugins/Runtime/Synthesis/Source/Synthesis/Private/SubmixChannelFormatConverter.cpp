// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubmixChannelFormatConverter.h"

#include "AudioMixerDevice.h"
#include "SynthesisModule.h"

namespace Audio
{
	namespace SubmixChannelFormatConverterPrivate 
	{
		// The set of all front channels
		static const TSet<EAudioMixerChannel::Type> FrontChannels(
			{
				EAudioMixerChannel::FrontLeft,
				EAudioMixerChannel::FrontRight,
				EAudioMixerChannel::FrontCenter,
				EAudioMixerChannel::FrontLeftOfCenter,
				EAudioMixerChannel::FrontRightOfCenter,
				EAudioMixerChannel::TopFrontLeft,
				EAudioMixerChannel::TopFrontRight,
				EAudioMixerChannel::TopFrontCenter
			}
		);

		// For a rear-channel-bleed, this map pairs all front channels with all
		// associated rear channels that should receive bled audio.
		static const TMap<EAudioMixerChannel::Type, TArray<EAudioMixerChannel::Type>> PairedRearChannelTypes =
		{
			{ EAudioMixerChannel::FrontLeft, { EAudioMixerChannel::BackLeft, EAudioMixerChannel::SideLeft, EAudioMixerChannel::TopBackLeft } },
			{ EAudioMixerChannel::FrontRight, { EAudioMixerChannel::BackRight, EAudioMixerChannel::SideRight, EAudioMixerChannel::TopBackRight } },
			{ EAudioMixerChannel::FrontCenter, { EAudioMixerChannel::BackCenter, EAudioMixerChannel::TopCenter, EAudioMixerChannel::TopBackCenter } },
			{ EAudioMixerChannel::FrontLeftOfCenter, { EAudioMixerChannel::BackLeft, EAudioMixerChannel::SideLeft, EAudioMixerChannel::TopBackLeft } },
			{ EAudioMixerChannel::FrontRightOfCenter, { EAudioMixerChannel::BackRight, EAudioMixerChannel::SideRight, EAudioMixerChannel::TopBackRight } },
			{ EAudioMixerChannel::TopFrontLeft, { EAudioMixerChannel::BackLeft, EAudioMixerChannel::SideLeft, EAudioMixerChannel::TopBackLeft } },
			{ EAudioMixerChannel::TopFrontRight, { EAudioMixerChannel::BackRight, EAudioMixerChannel::SideRight, EAudioMixerChannel::TopBackRight } },
			{ EAudioMixerChannel::TopFrontCenter, { EAudioMixerChannel::BackCenter, EAudioMixerChannel::TopCenter, EAudioMixerChannel::TopBackCenter } }
		};

		// For a _flipped_ rear-channel-bleed, this map pairs all front channels with all
		// associated rear channels that should receive bled audio.
		static const TMap<EAudioMixerChannel::Type, TArray<EAudioMixerChannel::Type>> PairedFlippedRearChannelTypes =
		{
			{ EAudioMixerChannel::FrontLeft, PairedRearChannelTypes[EAudioMixerChannel::FrontRight] },
			{ EAudioMixerChannel::FrontRight, PairedRearChannelTypes[EAudioMixerChannel::FrontLeft] },
			{ EAudioMixerChannel::FrontCenter, PairedRearChannelTypes[EAudioMixerChannel::FrontCenter] },
			{ EAudioMixerChannel::FrontLeftOfCenter, PairedRearChannelTypes[EAudioMixerChannel::FrontRightOfCenter] },
			{ EAudioMixerChannel::FrontRightOfCenter, PairedRearChannelTypes[EAudioMixerChannel::FrontLeftOfCenter] },
			{ EAudioMixerChannel::TopFrontLeft, PairedRearChannelTypes[EAudioMixerChannel::TopFrontRight] },
			{ EAudioMixerChannel::TopFrontRight, PairedRearChannelTypes[EAudioMixerChannel::TopFrontLeft] },
			{ EAudioMixerChannel::TopFrontCenter, PairedRearChannelTypes[EAudioMixerChannel::TopFrontCenter] }
		};
	}

	bool GetSubmixChannelOrderForNumChannels(int32 InNumChannels, TArray<EAudioMixerChannel::Type>& OutChannelOrder)
	{
		OutChannelOrder.Reset();

		// These are hardcoded here. If they're needed elsewhere, this bit of
		// code should be shuffled around somewhere convenient. 
		switch (InNumChannels)
		{
			case 1:
				
				// Mono
				OutChannelOrder.Append({EAudioMixerChannel::Type::FrontCenter});
				return true;

			case 2:

				// Stereo
				OutChannelOrder.Append({EAudioMixerChannel::Type::FrontLeft, EAudioMixerChannel::Type::FrontRight});
				return true;

			case 4:

				// Quad
				OutChannelOrder.Append(
					{
						EAudioMixerChannel::Type::FrontLeft,
						EAudioMixerChannel::Type::FrontRight,
						EAudioMixerChannel::Type::SideLeft,
						EAudioMixerChannel::Type::SideRight
					}
				);

				return true;

			case 6:

				// 5.1
				OutChannelOrder.Append(
					{
						EAudioMixerChannel::Type::FrontLeft,
						EAudioMixerChannel::Type::FrontRight,
						EAudioMixerChannel::Type::FrontCenter,
						EAudioMixerChannel::Type::LowFrequency,
						EAudioMixerChannel::Type::SideLeft,
						EAudioMixerChannel::Type::SideRight
					}
				);

				return true;

			case 8:

				// 7.1
				OutChannelOrder.Append(
					{
						EAudioMixerChannel::Type::FrontLeft,
						EAudioMixerChannel::Type::FrontRight,
						EAudioMixerChannel::Type::FrontCenter,
						EAudioMixerChannel::Type::LowFrequency,
						EAudioMixerChannel::Type::BackLeft,
						EAudioMixerChannel::Type::BackRight,
						EAudioMixerChannel::Type::SideLeft,
						EAudioMixerChannel::Type::SideRight
					}
				);

				return true;
		}

		return false;
	};


	bool FAC3DownmixerFactory::GetAC3DownmixEntries(const FInputFormat& InInputFormat, const FOutputFormat& InOutputFormat, TArray<FChannelMixEntry>& OutMixEntries)
	{
		OutMixEntries.Reset();

		if ((InInputFormat.NumChannels < 1) || (InOutputFormat.NumChannels < 1))
		{
			return false;
		}

		static const bool bIsVorbis = false;
		static const bool bIsCenterChannelOnly = false;

		// Get channel map from FMixerDevice
		FAlignedFloatBuffer ChannelMap;

		FMixerDevice::Get2DChannelMap(bIsVorbis, InInputFormat.NumChannels, InOutputFormat.NumChannels, bIsCenterChannelOnly, ChannelMap);

		const int32 ExpectedChannelMapSize = InInputFormat.NumChannels * InOutputFormat.NumChannels;
	
		if (ChannelMap.Num() != ExpectedChannelMapSize)
		{
			UE_LOGF(LogSynthesis, Error, "Unexpected channel map size of %d. Expected %d", ChannelMap.Num(), ExpectedChannelMapSize);
			return false;
		}

		// Build array of FChannelMixEntry's from ChannelMap.
		TArray<FChannelMixEntry> ChannelMixEntries;

		for (int32 InputIndex = 0; InputIndex < InInputFormat.NumChannels; InputIndex++)
		{
			for (int32 OutputIndex = 0; OutputIndex < InOutputFormat.NumChannels; OutputIndex++)
			{
				int32 Index = InputIndex * InOutputFormat.NumChannels + OutputIndex;

				if (ChannelMap[Index] != 0.f)
				{
					FChannelMixEntry& Entry = OutMixEntries.AddDefaulted_GetRef();

					Entry.InputChannelIndex = InputIndex;
					Entry.OutputChannelIndex = OutputIndex;
					Entry.Gain = ChannelMap[Index];
				}
			}
		}

		return true;
	}

	TUniquePtr<FBaseChannelFormatConverter> FAC3DownmixerFactory::CreateAC3Downmixer(const FInputFormat& InInputFormat, const FOutputFormat& InOutputFormat, int32 InNumFramesPerCall)
	{
		TArray<FChannelMixEntry> ChannelMixEntries;

		bool bSuccess = GetAC3DownmixEntries(InInputFormat, InOutputFormat, ChannelMixEntries);

		if (!bSuccess)
		{
			return TUniquePtr<FBaseChannelFormatConverter>(nullptr);	
		}

		return FBaseChannelFormatConverter::CreateBaseFormatConverter(InInputFormat, InOutputFormat, ChannelMixEntries, InNumFramesPerCall);
	}

	const FSimpleRouter::FInputFormat& FSimpleRouter::GetInputFormat() const
	{
		return InputFormat;
	}

	const FSimpleRouter::FOutputFormat& FSimpleRouter::GetOutputFormat() const
	{
		return OutputFormat;
	}
	
	void FSimpleRouter::ProcessAudio(const TArray<FAlignedFloatBuffer>& InInputBuffers, TArray<FAlignedFloatBuffer>& OutOutputBuffers)
	{
		check(InInputBuffers.Num() == InputFormat.NumChannels);

		// Ensure output buffers exist in output array.
		while (OutOutputBuffers.Num() < OutputFormat.NumChannels)
		{
			OutOutputBuffers.Emplace();
		}

		for (int32 i = 0; i < OutputFormat.NumChannels; i++)
		{
			// Allocate output buffers to the correct size. 
			OutOutputBuffers[i].Reset();
			OutOutputBuffers[i].AddUninitialized(NumFramesPerCall);

			// Zero out data in output buffers. 
			FMemory::Memset(OutOutputBuffers[i].GetData(), 0, sizeof(float) * NumFramesPerCall);
		}

		// Copy data over for matched channels
		for (const TPair<int32, int32>& ChannelPair : ChannelPairs)
		{
			const int32 InputIndex = ChannelPair.Get<0>();
			const int32 OutputIndex = ChannelPair.Get<1>();

			if (ensure(InputIndex < InInputBuffers.Num()))
			{
				FMemory::Memcpy(OutOutputBuffers[OutputIndex].GetData(), InInputBuffers[InputIndex].GetData(), sizeof(float) * NumFramesPerCall);
			}
		}

	}

	TUniquePtr<FSimpleRouter> FSimpleRouter::CreateSimpleRouter(const TArray<EAudioMixerChannel::Type>& InInputChannelTypes, const TArray<EAudioMixerChannel::Type>& InOutputChannelTypes, int32 InNumFramesPerCall)
	{
		// Check valid frame count;
		if (InNumFramesPerCall < 1)
		{
			UE_LOGF(LogSynthesis, Error, "Invalid number of frames per call [%d]", InNumFramesPerCall);
			return TUniquePtr<FSimpleRouter>();
		}

		// Check for duplicates
		TSet<EAudioMixerChannel::Type> InputChannelTypeSet(InInputChannelTypes);

		if (InputChannelTypeSet.Num() != InInputChannelTypes.Num())
		{
			UE_LOGF(LogSynthesis, Error, "Input channel type contains duplicate values.");
			return TUniquePtr<FSimpleRouter>();
		}

		TSet<EAudioMixerChannel::Type> OutputChannelTypeSet(InOutputChannelTypes);

		if (OutputChannelTypeSet.Num() != InOutputChannelTypes.Num())
		{
			UE_LOGF(LogSynthesis, Error, "Output channel type contains duplicate values.");
			return TUniquePtr<FSimpleRouter>();
		}

		return TUniquePtr<FSimpleRouter>(new FSimpleRouter(InInputChannelTypes, InOutputChannelTypes, InNumFramesPerCall));
	}


	FSimpleRouter::FSimpleRouter(const TArray<EAudioMixerChannel::Type>& InInputChannelTypes, const TArray<EAudioMixerChannel::Type>& InOutputChannelTypes, int32 InNumFramesPerCall)
	:	InputChannelTypes(InInputChannelTypes)
	,	OutputChannelTypes(InOutputChannelTypes)
	,	NumFramesPerCall(InNumFramesPerCall)
	{
		InputFormat.NumChannels = InputChannelTypes.Num();
		OutputFormat.NumChannels = OutputChannelTypes.Num();

		check(NumFramesPerCall > 0);

		// Default routing is to match channel types.
		for (int32 InputIndex = 0; InputIndex < InputChannelTypes.Num(); InputIndex++)
		{
			EAudioMixerChannel::Type InputType = InputChannelTypes[InputIndex];
			int32 OutputIndex = OutputChannelTypes.Find(InputType);

			if (OutputIndex != INDEX_NONE)
			{
				ChannelPairs.Emplace(InputIndex, OutputIndex);
			}
		}

		// Determine special case of routing mono to front left. Only do special mapping
		// if input is mono-front-center and output has front-left but no front-center.
		const bool bIsInputMono = (InInputChannelTypes.Num() == 1) && (InInputChannelTypes[0] == EAudioMixerChannel::FrontCenter);
		const bool bIsFrontCenterInOutput = InOutputChannelTypes.Find(EAudioMixerChannel::FrontCenter) != INDEX_NONE;
		const bool bIsFrontLeftInOutput = InOutputChannelTypes.Find(EAudioMixerChannel::FrontLeft) != INDEX_NONE;

		const bool bMapInputFrontCenterToOutputFrontLeft = bIsInputMono && bIsFrontLeftInOutput && !bIsFrontCenterInOutput;

		if (bMapInputFrontCenterToOutputFrontLeft)
		{
			ChannelPairs.Emplace(InInputChannelTypes.Find(EAudioMixerChannel::FrontCenter), InOutputChannelTypes.Find(EAudioMixerChannel::FrontLeft));
		}
	}

	void FSimpleUpmixer::SetRearChannelBleed(float InGain, bool bFadeToGain)
	{
		for (int32 InputChannelIndex : FrontChannelIndices)
		{
			const TArray<int32>& RearChannelIndices = GetPairedRearChannelIndices(InputChannelIndex);

			for (int32 RearChannelIndex : RearChannelIndices)
			{
				SetMixGain(InputChannelIndex, RearChannelIndex, InGain, bFadeToGain);
			}
		}

		// Output gain needs to be updated so things don't get too loud.
		UpdateOutputGain(bFadeToGain);
	}

	void FSimpleUpmixer::SetRearChannelFlip(bool bInDoRearChannelFlip, bool bFadeFlip)
	{
		// Only process on change in value. 
		if (bDoRearChannelFlip != bInDoRearChannelFlip)
		{
			// Setting the rear channel flip will update the `GetPairedRearChannelIndices` 
			// results.  Cache the existing rear channel gains so they can be used after 
			// the rear channel swap.

			TArray<TArray<float>> ExistingGains;

			for (int32 InputChannelIndex : FrontChannelIndices)
			{
				// Get rear channel indices before toggling `bDoRearChannelFlip`
				const TArray<int32>& RearChannelIndices = GetPairedRearChannelIndices(InputChannelIndex);

				TArray<float> Gains;

				for (int32 RearChannelIndex : RearChannelIndices)
				{
					// Cache existing gain.
					Gains.Add(GetTargetMixGain(InputChannelIndex, RearChannelIndex));

					// Clear out the existing gain.
					SetMixGain(InputChannelIndex, RearChannelIndex, 0.f, bFadeFlip);
				}

				ExistingGains.Add(Gains);
			}

			// Toggling bDoRearChannelFlip will alter the behavior of `GetPairedRearChannelIndices`.
			bDoRearChannelFlip = bInDoRearChannelFlip;

			for (int32 i = 0; i < ExistingGains.Num(); i++)
			{
				int32 InputChannelIndex = FrontChannelIndices[i];

				// Get rear channel indices after toggling `bDoRearChannelFlip`
				const TArray<int32>& RearChannelIndices = GetPairedRearChannelIndices(InputChannelIndex);

				for (int32 j = 0; j < RearChannelIndices.Num(); j++)
				{
					int32 RearChannelIndex = RearChannelIndices[j];
					float RearChannelGain = ExistingGains[i][j];

					// Set new mix gain after toggling bDoRearChannelFlip
					SetMixGain(InputChannelIndex, RearChannelIndex, RearChannelGain, bFadeFlip);
				}
			}

			// Output gain changes to account for correlation between signals
			// when downmixed. 
			UpdateOutputGain(bFadeFlip);
		}
	}

	bool FSimpleUpmixer::GetRearChannelFlip() const
	{
		return bDoRearChannelFlip;
	}


	bool FSimpleUpmixer::GetSimpleUpmixerStaticMixEntries(const TArray<EAudioMixerChannel::Type>& InInputChannelTypes, const TArray<EAudioMixerChannel::Type>& InOutputChannelTypes, TArray<FChannelMixEntry>& OutEntries)
	{
		OutEntries.Reset();

		// Query which input channels exist
		const int32 InputFrontLeftIndex = InInputChannelTypes.Find(EAudioMixerChannel::Type::FrontLeft);
		const int32 InputFrontRightIndex = InInputChannelTypes.Find(EAudioMixerChannel::Type::FrontRight);
		const int32 InputSideLeftIndex = InInputChannelTypes.Find(EAudioMixerChannel::Type::SideLeft);
		const int32 InputSideRightIndex = InInputChannelTypes.Find(EAudioMixerChannel::Type::SideRight);
		const int32 InputBackLeftIndex = InInputChannelTypes.Find(EAudioMixerChannel::Type::BackLeft);
		const int32 InputBackRightIndex = InInputChannelTypes.Find(EAudioMixerChannel::Type::BackRight);

		// Query which output channels exist
		const int32 OutputFrontLeftIndex = InOutputChannelTypes.Find(EAudioMixerChannel::Type::FrontLeft);
		const int32 OutputFrontRightIndex = InOutputChannelTypes.Find(EAudioMixerChannel::Type::FrontRight);
		const int32 OutputSideLeftIndex = InOutputChannelTypes.Find(EAudioMixerChannel::Type::SideLeft);
		const int32 OutputSideRightIndex = InOutputChannelTypes.Find(EAudioMixerChannel::Type::SideRight);
		const int32 OutputBackLeftIndex = InOutputChannelTypes.Find(EAudioMixerChannel::Type::BackLeft);
		const int32 OutputBackRightIndex = InOutputChannelTypes.Find(EAudioMixerChannel::Type::BackRight);

		// Check for mono input and stereo output.
		const bool bInputIsMono = (1 == InInputChannelTypes.Num()) && (InInputChannelTypes[0] == EAudioMixerChannel::Type::FrontCenter); 
		const bool bOutputHasFrontLeft = (INDEX_NONE != OutputFrontLeftIndex);
		const bool bOutputHasFrontRight = (INDEX_NONE != OutputFrontRightIndex);

		const bool bUpmixMono = bInputIsMono && (bOutputHasFrontLeft || bOutputHasFrontRight);

		// Check for upmixing quad or 5.1 to 7.1
		// 7.1 has rear and side channels, while 5.1 has side channels but no rear channels.
		const bool bInputHasSideLeft = (INDEX_NONE != InputSideLeftIndex);
		const bool bInputHasSideRight = (INDEX_NONE != InputSideRightIndex);
		const bool bInputHasSide = bInputHasSideLeft || bInputHasSideRight;
		const bool bInputHasBackLeft = (INDEX_NONE != InputBackLeftIndex);
		const bool bInputHasBackRight = (INDEX_NONE != InputBackRightIndex);
		const bool bInputHasBack = bInputHasBackLeft || bInputHasBackRight;

		const bool bOutputHasSideLeft = (INDEX_NONE != OutputSideLeftIndex);
		const bool bOutputHasSideRight = (INDEX_NONE != OutputSideRightIndex);
		const bool bOutputHasSide = bOutputHasSideLeft || bOutputHasSideRight;
		const bool bOutputHasBackLeft = (INDEX_NONE != OutputBackLeftIndex);
		const bool bOutputHasBackRight = (INDEX_NONE != OutputBackRightIndex);
		const bool bOutputHasBack = bOutputHasBackLeft || bOutputHasBackRight;

		const bool bUpmixSideToBack = bInputHasSide && !bInputHasBack && bOutputHasBack;

		if (!bUpmixMono)
		{
			// If not upmixing mono, then channels are matched if they have the same
			// EAudioMixerChannel::Type type. 
			for (int32 InputChannelIndex = 0; InputChannelIndex < InInputChannelTypes.Num(); InputChannelIndex++)
			{
				int32 OutputChannelIndex = InOutputChannelTypes.Find(InInputChannelTypes[InputChannelIndex]);

				if (INDEX_NONE != OutputChannelIndex)
				{
					FChannelMixEntry& Entry = OutEntries.AddDefaulted_GetRef();

					Entry.InputChannelIndex = InputChannelIndex;
					Entry.OutputChannelIndex = OutputChannelIndex;
					Entry.Gain = 1.f;
				}
				else
				{
					// This likely means audio gets dropped instead of mixed. 
					UE_LOGF(LogSynthesis, Warning, "Unable to match input channel [%ls] to output channel", EAudioMixerChannel::ToString(InInputChannelTypes[InputChannelIndex]));
				}
			}

			// upmix side to rear channels. 
			if (bUpmixSideToBack)
			{
				// If output has side and rear, mix equal amplitude to account
				// for possible downmixing from 7.1 bed to 5.1.

				// Upmix side left to rear left
				if (bInputHasSideLeft && bOutputHasBackLeft)
				{
					FChannelMixEntry& Entry = OutEntries.AddDefaulted_GetRef();

					Entry.InputChannelIndex = InputSideLeftIndex;
					Entry.OutputChannelIndex = OutputBackLeftIndex;

					if (bOutputHasSideLeft)
					{
						Entry.Gain = 0.5f;

						// Find existing gain and reduce to handle downmixing scenario
						for (FChannelMixEntry& ExistingEntry : OutEntries)
						{
							if ((ExistingEntry.InputChannelIndex == InputSideLeftIndex) && (ExistingEntry.OutputChannelIndex == OutputSideLeftIndex))
							{
								ExistingEntry.Gain = 0.5;
							}
						}
					}
					else
					{
						Entry.Gain = 1.0f;
					}
				}

				// Upmix side right to rear right
				if (bInputHasSideRight && bOutputHasBackRight)
				{
					FChannelMixEntry& Entry = OutEntries.AddDefaulted_GetRef();

					Entry.InputChannelIndex = InputSideRightIndex;
					Entry.OutputChannelIndex = OutputBackRightIndex;

					if (bOutputHasSideRight)
					{
						Entry.Gain = 0.5f;

						// Find existing gain and reduce to handle downmixing scenario
						for (FChannelMixEntry& ExistingEntry : OutEntries)
						{
							if ((ExistingEntry.InputChannelIndex == InputSideRightIndex) && (ExistingEntry.OutputChannelIndex == OutputSideRightIndex))
							{
								ExistingEntry.Gain = 0.5;
							}
						}
					}
					else
					{
						Entry.Gain = 1.0f;
					}
				}
			}
		}
		else 
		{
			// Handle upmixing mono to output format. 
			const bool bUpmixMonoToStereo = bInputIsMono && bOutputHasFrontLeft && bOutputHasFrontRight;

			if (bUpmixMonoToStereo)
			{
				// If there is mono input and at least 2 output channels, upmix
				// to stereo. 
				FChannelMixEntry& MonoToFrontLeft = OutEntries.AddDefaulted_GetRef();

				MonoToFrontLeft.InputChannelIndex = 0;
				MonoToFrontLeft.OutputChannelIndex = InOutputChannelTypes.Find(EAudioMixerChannel::Type::FrontLeft);
				MonoToFrontLeft.Gain = 0.707f; // Equal power

				FChannelMixEntry& MonoToFrontRight = OutEntries.AddDefaulted_GetRef();

				MonoToFrontRight.InputChannelIndex = 0;
				MonoToFrontRight.OutputChannelIndex = InOutputChannelTypes.Find(EAudioMixerChannel::Type::FrontRight);
				MonoToFrontRight.Gain = 0.707f; // Equal power
			}
			else
			{
				// If there is mono and only a left or right, route mono to whichever
				// one is present.
				FChannelMixEntry& MonoToFront = OutEntries.AddDefaulted_GetRef();
				MonoToFront.InputChannelIndex = 0;
				MonoToFront.Gain = 1.f;

				if (bOutputHasFrontLeft)
				{
					MonoToFront.OutputChannelIndex = InOutputChannelTypes.Find(EAudioMixerChannel::Type::FrontLeft);
				}
				else
				{
					MonoToFront.OutputChannelIndex = InOutputChannelTypes.Find(EAudioMixerChannel::Type::FrontRight);
				}
			}
		}

		return true;
	}

	TUniquePtr<FSimpleUpmixer> FSimpleUpmixer::CreateSimpleUpmixer(const TArray<EAudioMixerChannel::Type>& InInputChannelTypes, const TArray<EAudioMixerChannel::Type>& InOutputChannelTypes, int32 InNumFramesPerCall)
	{
		if (InInputChannelTypes.Num() < 1)
		{
			UE_LOGF(LogSynthesis, Error, "Invalid input format channel count (%d). Must be greater than zero", InInputChannelTypes.Num());
			return TUniquePtr<FSimpleUpmixer>(nullptr);
		}

		if (InOutputChannelTypes.Num() < 1) 
		{
			UE_LOGF(LogSynthesis, Error, "Invalid output format channel count (%d). Must be greater than zero", InOutputChannelTypes.Num());
			return TUniquePtr<FSimpleUpmixer>(nullptr);
		}

		if (InNumFramesPerCall < 1)
		{
			UE_LOGF(LogSynthesis, Error, "Invalid num frames per call (%d). Must be greater than zero", InNumFramesPerCall);
			return TUniquePtr<FSimpleUpmixer>(nullptr);
		}

		TArray<FChannelMixEntry> ChannelMixEntries;

		bool bSuccess = GetSimpleUpmixerStaticMixEntries(InInputChannelTypes, InOutputChannelTypes, ChannelMixEntries);

		if (!bSuccess)
		{
			return TUniquePtr<FSimpleUpmixer>(nullptr);
		}

		return TUniquePtr<FSimpleUpmixer>(new FSimpleUpmixer(InInputChannelTypes, InOutputChannelTypes, ChannelMixEntries, InNumFramesPerCall));
	}

	FSimpleUpmixer::FSimpleUpmixer(const TArray<EAudioMixerChannel::Type>& InInputChannelTypes, const TArray<EAudioMixerChannel::Type>& InOutputChannelTypes, TArrayView<const FChannelMixEntry> InMixEntries, int32 InNumFramesPerCall)
	:	FBaseChannelFormatConverter({InInputChannelTypes.Num()}, {InOutputChannelTypes.Num()}, InMixEntries, InNumFramesPerCall)
	,	InputChannelTypes(InInputChannelTypes)
	,	OutputChannelTypes(InOutputChannelTypes)
	{
		// Cache front channel indices. These do not change for lifetime of FSimpleUpmixer.
		InitFrontChannelIndices(FrontChannelIndices);

		// Cache rear channel indices.
		for (int32 FrontChannelIndex : FrontChannelIndices)
		{
			TArray<int32>& RearChannelIndices = PairedRearChannelIndices.Add(FrontChannelIndex);

			InitPairedRearChannelIndices(FrontChannelIndex, false /* bFlipped */, RearChannelIndices);

			TArray<int32>& FlippedRearChannelIndices = PairedFlippedRearChannelIndices.Add(FrontChannelIndex);

			InitPairedRearChannelIndices(FrontChannelIndex, true /* bFlipped */, FlippedRearChannelIndices);
		}
	}

	void FSimpleUpmixer::UpdateOutputGain(bool bFadeToGain)
	{
		// Update output gain to keep overall loudness constant if later mixed down.
		int32 NumRearChannelBleed = 0;
		float SumRearChannelBleed = 0.f;

		for (int32 InputChannelIndex : FrontChannelIndices)
		{
			const TArray<int32>& RearChannelIndices = GetPairedRearChannelIndices(InputChannelIndex);

			for (int32 RearChannelIndex : RearChannelIndices)
			{
				SumRearChannelBleed = FMath::Abs(GetTargetMixGain(InputChannelIndex, RearChannelIndex));
				NumRearChannelBleed++;
			}
		}

		if (0 == NumRearChannelBleed)
		{
			SetOutputGain(1.f, bFadeToGain);
		}
		else
		{
			float AverageRearGain = SumRearChannelBleed / 2.f; // Normalize for stereo mixdown.
			float OutputGain = 1.f;

			if (bDoRearChannelFlip)
			{
				// If rear channel flip, can assume that signals are uncorrelated if they are mixed back down,
				// so use equal power normalization.
				OutputGain = 1.f / FMath::Sqrt(1.f + (0.707f * 0.707f * AverageRearGain * AverageRearGain));
			}
			else
			{
				// Rear channels not flipped, so assume signals are correlated if they are mixed back down,
				// so use equal amplitude normalization. The 0.707f comes from the
				// values used to do mixdowns from surround to stereo.
				OutputGain = 1.f / FMath::Max(1.f, (1.f + 0.707f * AverageRearGain));
			}

			// Make sure output gain is less than or equal to 1
			OutputGain = FMath::Min(1.f, OutputGain);

			SetOutputGain(OutputGain, bFadeToGain);
		}
	}

	void FSimpleUpmixer::InitFrontChannelIndices(TArray<int32>& OutFrontChannelIndices) const
	{
		using namespace SubmixChannelFormatConverterPrivate;

		OutFrontChannelIndices.Reset();

		for (int32 ChannelIndex = 0; ChannelIndex < InputChannelTypes.Num(); ChannelIndex++)
		{
			if (FrontChannels.Contains(InputChannelTypes[ChannelIndex]))
			{
				OutFrontChannelIndices.Add(ChannelIndex);
			}
		}
	}

	void FSimpleUpmixer::InitPairedRearChannelIndices(int32 InInputChannelIndex, bool bInFlipped, TArray<int32>& OutRearChannelIndices) const
	{
		using namespace SubmixChannelFormatConverterPrivate;

		OutRearChannelIndices.Reset();

		EAudioMixerChannel::Type InputChannelType = InputChannelTypes[InInputChannelIndex];
		
		const TMap<EAudioMixerChannel::Type, TArray<EAudioMixerChannel::Type>>& ChannelTypeMap = bInFlipped ? PairedFlippedRearChannelTypes : PairedRearChannelTypes;

		if (ChannelTypeMap.Contains(InputChannelType))
		{
			const TArray<EAudioMixerChannel::Type>& PairedChannelTypes = ChannelTypeMap[InputChannelType];

			for (int32 OutputChannelIndex = 0; OutputChannelIndex < OutputChannelTypes.Num(); OutputChannelIndex++)
			{
				if (PairedChannelTypes.Contains(OutputChannelTypes[OutputChannelIndex]))
				{
					OutRearChannelIndices.Add(OutputChannelIndex);
				}
			}
		}
	}

	const TArray<int32>& FSimpleUpmixer::GetPairedRearChannelIndices(int32 InInputChannelIndex) const
	{
		static const TArray<int32> EmptyArray;

		if (bDoRearChannelFlip)
		{
			if (const TArray<int32>* RearIndices = PairedFlippedRearChannelIndices.Find(InInputChannelIndex))
			{
				return *RearIndices;
			}
		}
		else
		{
			if (const TArray<int32>* RearIndices = PairedRearChannelIndices.Find(InInputChannelIndex))
			{
				return *RearIndices;
			}
		}

		return EmptyArray;
	}
}
