// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioMixer.h"
#include "DSP/AudioChannelFormatConverter.h"

namespace Audio
{
	/** Returns the defualt channeel order for a given number of submix channels.
	 *
	 * @param InNumChannels - The number of channels in the submix.
	 * @param OutChannelOrder - An array of channel types denoting the channel order.
	 *
	 * @return True on success, false on error.
	 */
	bool GetSubmixChannelOrderForNumChannels(int32 InNumChannels, TArray<EAudioMixerChannel::Type>& OutChannelOrder);

	/** Factory for creating downmixers based on the AC3 downmixing gain values. */
	struct FAC3DownmixerFactory
	{
		using FChannelMixEntry = FBaseChannelFormatConverter::FChannelMixEntry;
		using FInputFormat = IChannelFormatConverter::FInputFormat;
		using FOutputFormat = IChannelFormatConverter::FOutputFormat;

		/** Populate an array of mix entries to go from an input format to an output format using AC3 conforming gains. 
		 *
		 * @param InInputFormat - The format of the input audio.
		 * @param InOutputFormat - The desired output audio format.
		 * @param OutMixEntries - An array of channel mix entries. 
		 *
		 * @return True on success, false on failure. 
		 */
		static bool GetAC3DownmixEntries(const FInputFormat& InInputFormat, const FOutputFormat& InOutputFormat, TArray<FChannelMixEntry>& OutMixEntries);

		/** Creates an FBaseChannelFormatConverter using AC3 downmixing weights
		 * to convert the InInputFormat to the InOutputFormat.
		 *
		 * @param InInputFormat - The format of the input audio.
		 * @param InOutputFormat - The desired output audio format.
		 * @param InNumFramesPerCall - The number of frames used for each call to ProcessAudio.
		 *
		 * @return A TUniquePtr to a FBaseChannelFormatConverter.
		 */
		static TUniquePtr<FBaseChannelFormatConverter> CreateAC3Downmixer(const FInputFormat& InInputFormat, const FOutputFormat& InOutputFormat, int32 InNumFramesPerCall);
	};

	class FSimpleRouter : public IChannelFormatConverter
	{
		public:

			using FInputFormat = IChannelFormatConverter::FInputFormat;
			using FOutputFormat = IChannelFormatConverter::FOutputFormat;

			virtual ~FSimpleRouter() = default;

			/** Return the input format handled by this converter. */
			const FInputFormat& GetInputFormat() const override;

			/** Return the output format handled by this converter. */
			const FOutputFormat& GetOutputFormat() const override;
			
			/** Converter the audio format from the FInputFormat to the FOutputFormat. 
			 *
			 * The input buffer array must have the same number of channels as the
			 * FInputFormat return from GetInputFormat().  Each buffer within that 
			 * array must have the same number of samples.
			 *
			 * @param InInputBuffers - An array of input audio buffers.
			 * @param OutOutputBuffers - An array of buffers where output audio is stored. 
			 */
			void ProcessAudio(const TArray<FAlignedFloatBuffer>& InInputBuffers, TArray<FAlignedFloatBuffer>& OutOutputBuffers) override;

			/** Create a simple router.
			 *
			 * The simple router copies input data to output data if the channel 
			 * types match for the input and output channel types.
			 *
			 * @param InInputChannelTypes - Array denoting input channel type order.
			 * @param InOutputChannelTypes - Array denoting input channel type order.
			 * @param InNumFramesPerCall - Number of frames used for each call to ProcessAudio.
			 *
			 * @return A valid TUniquePtr<FSimpleRouter> on success. An invalid pointer on failure. 
			 */
			static TUniquePtr<FSimpleRouter> CreateSimpleRouter(const TArray<EAudioMixerChannel::Type>& InInputChannelTypes, const TArray<EAudioMixerChannel::Type>& InOutputChannelTypes, int32 InNumFramesPerCall);

		protected:

			FSimpleRouter(const TArray<EAudioMixerChannel::Type>& InInputChannelTypes, const TArray<EAudioMixerChannel::Type>& InOutputChannelTypes, int32 InNumFramesPerCall);

		private:

			FInputFormat InputFormat;
			FOutputFormat OutputFormat;

			TArray<EAudioMixerChannel::Type> InputChannelTypes;
			TArray<EAudioMixerChannel::Type> OutputChannelTypes;

			int32 NumFramesPerCall;

			TArray<TPair<int32, int32>> ChannelPairs;
	};

	/** FSimpleUpmixer handles a handful of upmixing scenarios.  Mono to Stereo,
	 * Mono to Surround and Stereo to Surround.
	 *
	 * When upmixing from stereo to surround, front left and front right audio 
	 * is mixed to the rear left and rear right audio in accordance with the set
	 * RearChannelBleep and RearChannelFlip.
	 */
	class FSimpleUpmixer : public FBaseChannelFormatConverter 
	{
		public:
			
			virtual ~FSimpleUpmixer() = default;

			/** Sets amount of mixing from front channels to paired rear channels.
			 *
			 * @param InGain - Scalar gain to apply front channels when adding them to rear channels. 
			 * @param bFadeToGain - If true, gain values are linearly faded over 
			 *                      the duration of a single buffer. If false,
			 *                      gain values are applied immediately. 
			 */
			void SetRearChannelBleed(float InGain, bool bFadeToGain=true);

			/** Set whether left and right rear channels should be swapped.
			 *
			 * @param bInDoRearChannelFlip - If true, front left is paired with rear right and vice-a-versa. 
			 *                        		 If false, front left is paired with rear left and vice-a-versa.
			 * @param bFadeToGain - If true, gain values are linearly faded over 
			 *                      the duration of a single buffer. If false,
			 *                      gain values are applied immediately. 
			 */
			void SetRearChannelFlip(bool bInDoRearChannelFlip, bool bFadeFlip=true);

			/** Returns whether the rear channel is flipped. */
			bool GetRearChannelFlip() const;

			/** Gets the unchanging mix entries for the simple upmixer.  
			 *
			 * These mix entries are generally unchanged during the lifetime of a simple upmixer.
			 *
			 * @param InInputChannelTypes - An array of channels describing the input channel order.
			 * @param InOutputChannelTypes - An array of channels describing the output channel order. 
			 * @param OutEntries - An array of channel mix entries. 
			 *
			 * @return True on success, false on failure. 
			 */
			static bool GetSimpleUpmixerStaticMixEntries(const TArray<EAudioMixerChannel::Type>& InInputChannelTypes, const TArray<EAudioMixerChannel::Type>& InOutputChannelTypes, TArray<FChannelMixEntry>& OutEntries);

			/** Creates an FSimpleUpmixer. 
			 *
			 *
			 * @param InInputChannelTypes - An array of channels describing the input channel order.
			 * @param InOutputChannelTypes - An array of channels describing the output channel order. 
			 * @param InNumFramesPerCall - The number of frames used for each call to ProcessAudio.
			 */
			static TUniquePtr<FSimpleUpmixer> CreateSimpleUpmixer(const TArray<EAudioMixerChannel::Type>& InInputChannelTypes, const TArray<EAudioMixerChannel::Type>& InOutputChannelTypes, int32 InNumFramesPerCall);

		protected:

			FSimpleUpmixer(const TArray<EAudioMixerChannel::Type>& InInputChannelTypes, const TArray<EAudioMixerChannel::Type>& InOutputChannelTypes, TArrayView<const FChannelMixEntry> InMixEntries, int32 InNumFramesPerCall);

		private:

			void UpdateOutputGain(bool bFadeToGain);

			void InitFrontChannelIndices(TArray<int32>& OutFrontChannelIndices) const;

			// Returns the channels associated with a rear channel indices. 
			void InitPairedRearChannelIndices(int32 InInputChannelIndex, bool bInFlipped, TArray<int32>& OutRearChannelIndices) const;

			const TArray<int32>& GetPairedRearChannelIndices(int32 InInputChannelIndex) const;

			TArray<EAudioMixerChannel::Type> InputChannelTypes;
			TArray<EAudioMixerChannel::Type> OutputChannelTypes;

			TArray<int32> FrontChannelIndices;
			TSortedMap<int32, TArray<int32>> PairedRearChannelIndices;
			TSortedMap<int32, TArray<int32>> PairedFlippedRearChannelIndices;

			bool bDoRearChannelFlip = false;
	};
}
