// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubmixEffects/SubmixEffectConvolutionReverb.h"

#include "AudioDefines.h"
#include "DSP/AlignedBlockBuffer.h" // IWYU pragma: keep
#include "DSP/FloatArrayMath.h"
#include "EffectConvolutionReverb.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SubmixEffectConvolutionReverb)

FSubmixEffectConvolutionReverbSettings::FSubmixEffectConvolutionReverbSettings()
	: NormalizationVolumeDb(-24.f)
	, WetVolumeDb(0.f)
	, DryVolumeDb(-96.f)
	, bBypass(false)
	, bMixInputChannelFormatToImpulseResponseFormat(true)
	, bMixReverbOutputToOutputChannelFormat(true)
	, SurroundRearChannelBleedDb(-60.f)
	, bInvertRearChannelBleedPhase(false)
	, bSurroundRearChannelFlip(false)
	, SurroundRearChannelBleedAmount_DEPRECATED(0.0f)
	, ImpulseResponse_DEPRECATED(nullptr)
	, AllowHardwareAcceleration_DEPRECATED(true)
{
}

FSubmixEffectConvolutionReverb::FSubmixEffectConvolutionReverb(const USubmixEffectConvolutionReverbPreset* InPreset)
	: Reverb(MakeShared<Audio::FEffectConvolutionReverb, ESPMode::ThreadSafe>())
{
	UpdateConvolutionReverb(InPreset);

	if (nullptr != InPreset)
	{
		SetConvolutionReverbParameters(InPreset->GetSettings());
	}
}

FSubmixEffectConvolutionReverb::~FSubmixEffectConvolutionReverb()
{
}

void FSubmixEffectConvolutionReverb::Init(const FSoundEffectSubmixInitData& InitData)
{
	using namespace Audio;

	Reverb->SetSampleRate(InitData.SampleRate);

	Reverb->UpdateVersion();
	Reverb->Init();
}

void FSubmixEffectConvolutionReverb::OnPresetChanged()
{
	USubmixEffectConvolutionReverbPreset* ConvolutionPreset = CastChecked<USubmixEffectConvolutionReverbPreset>(Preset);
	FSubmixEffectConvolutionReverbSettings Settings = ConvolutionPreset->GetSettings();;

	// Copy settings from FSubmixEffectConvolutionReverbSettings needed for FConvolutionReverbSettings 
	// FConvolutionReverbSettings represents runtime settings which do not need the 
	// FConvlutionReverb object to be rebuilt. Some settings in FSubmixEffectConvolutionReverbSettings
	// force a rebuild of FConvolutionReverb-> Those are handled in USubmixEffectConvolutinReverbPreset::PostEditChangeProperty
	SetConvolutionReverbParameters(Settings);
}

void FSubmixEffectConvolutionReverb::SetConvolutionReverbParameters(const FSubmixEffectConvolutionReverbSettings& InSettings)
{
	Audio::FConvolutionReverbSettings ReverbSettings;

	float NewRearChannelBleed = Audio::ConvertToLinear(InSettings.SurroundRearChannelBleedDb);

	if (InSettings.bInvertRearChannelBleedPhase)
	{
		NewRearChannelBleed *= -1.f;
	}

	ReverbSettings.RearChannelBleed = NewRearChannelBleed;

	ReverbSettings.NormalizationVolume = Audio::ConvertToLinear(InSettings.NormalizationVolumeDb);
	ReverbSettings.bRearChannelFlip = InSettings.bSurroundRearChannelFlip;

	WetVolume = Audio::ConvertToLinear(InSettings.WetVolumeDb);
	DryVolume = Audio::ConvertToLinear(InSettings.DryVolumeDb);

	Reverb->SetSettings(ReverbSettings);
	Reverb->SetBypass(InSettings.bBypass);
}

void FSubmixEffectConvolutionReverb::OnProcessAudio(const FSoundEffectSubmixInputData& InData, FSoundEffectSubmixOutputData& OutData)
{
	using namespace Audio;

	check(nullptr != InData.AudioBuffer);
	check(nullptr != OutData.AudioBuffer);
	check(InData.NumChannels != 0);

	Reverb->UpdateChannelCount(InData.NumChannels, OutData.NumChannels);

	Reverb->ProcessAudio(InData.NumChannels, InData.AudioBuffer->GetData(), OutData.NumChannels, OutData.AudioBuffer->GetData(), InData.NumFrames);

	// Process Wet/Dry mix
	if (FMath::IsNearlyEqual(WetVolume, 1.f) == false)
	{
		Audio::ArrayMultiplyByConstantInPlace(*OutData.AudioBuffer, WetVolume);
	}
	if (DryVolume > KINDA_SMALL_NUMBER)
	{
		Audio::ArrayMixIn(*InData.AudioBuffer, *OutData.AudioBuffer, DryVolume);
	}
}

AudioConvReverbIntrinsics::FVersionData FSubmixEffectConvolutionReverb::UpdateConvolutionReverb(const USubmixEffectConvolutionReverbPreset* InPreset)
{
	using namespace Audio;

	// Copy data from preset into internal FConvolutionReverbInitData
	// Reset data
	FConvolutionReverbInitData InitData = FConvolutionReverbInitData();

	if (nullptr != InPreset)
	{
		if (InPreset->ImpulseResponse)
		{
			UAudioImpulseResponse* IR = InPreset->ImpulseResponse;

			InitData.Samples = IR->ImpulseResponse;
			InitData.AlgorithmSettings.NumImpulseResponses = IR->NumChannels;
			InitData.ImpulseSampleRate = IR->SampleRate;
			InitData.bIsImpulseTrueStereo = IR->bTrueStereo && (IR->NumChannels % 2 == 0);
			InitData.NormalizationVolume = ConvertToLinear(IR->NormalizationVolumeDb);
		}

		switch (InPreset->BlockSize)
		{
		case ESubmixEffectConvolutionReverbBlockSize::BlockSize256:
			InitData.AlgorithmSettings.BlockNumSamples = 256;
			break;

		case ESubmixEffectConvolutionReverbBlockSize::BlockSize512:
			InitData.AlgorithmSettings.BlockNumSamples = 512;
			break;

		case ESubmixEffectConvolutionReverbBlockSize::BlockSize1024:
		default:
			InitData.AlgorithmSettings.BlockNumSamples = 1024;
			break;
		}

		InitData.AlgorithmSettings.bEnableHardwareAcceleration = InPreset->bEnableHardwareAcceleration;
		InitData.bMixInputChannelFormatToImpulseResponseFormat = InPreset->Settings.bMixInputChannelFormatToImpulseResponseFormat;
		InitData.bMixReverbOutputToOutputChannelFormat = InPreset->Settings.bMixReverbOutputToOutputChannelFormat;
	}

	Reverb->SetInitData(InitData);
	return Reverb->UpdateVersion();
}

void FSubmixEffectConvolutionReverb::RebuildConvolutionReverb()
{
	Reverb->BuildReverb();
}

/********************************************************************/
/***************** USubmixEffectConvolutionReverbPreset *************/
/********************************************************************/

USubmixEffectConvolutionReverbPreset::USubmixEffectConvolutionReverbPreset(const FObjectInitializer& ObjectInitializer)
:	Super(ObjectInitializer)
,	ImpulseResponse(nullptr)
,	BlockSize(ESubmixEffectConvolutionReverbBlockSize::BlockSize1024)
,	bEnableHardwareAcceleration(true)
{}

bool USubmixEffectConvolutionReverbPreset::CanFilter() const 
{
	return false; 
}

bool USubmixEffectConvolutionReverbPreset::HasAssetActions() const 
{ 
	return true; 
}

FText USubmixEffectConvolutionReverbPreset::GetAssetActionName() const 
{ 
	return FText::FromString(TEXT("SubmixEffectConvolutionReverb")); 
}

UClass* USubmixEffectConvolutionReverbPreset::GetSupportedClass() const 
{ 
	return USubmixEffectConvolutionReverbPreset::StaticClass(); 
}

FSoundEffectBase* USubmixEffectConvolutionReverbPreset::CreateNewEffect() const 
{ 
	// Pass a pointer to self into this constructor.  This enables the submix effect to have
	// a copy of the impulse response data. The submix effect can then prepare the impulse
	// response during it's "Init" routine.
	return new FSubmixEffectConvolutionReverb(this); 
}

USoundEffectPreset* USubmixEffectConvolutionReverbPreset::CreateNewPreset(UObject* InParent, FName Name, EObjectFlags Flags) const
{ 
	USoundEffectPreset* NewPreset = NewObject<USubmixEffectConvolutionReverbPreset>(InParent, GetSupportedClass(), Name, Flags); 
	NewPreset->Init(); 
	return NewPreset; 
}

void USubmixEffectConvolutionReverbPreset::Init() 
{ 
	FScopeLock ScopeLock(&SettingsCritSect);
	SettingsCopy = Settings;
}

void USubmixEffectConvolutionReverbPreset::UpdateSettings()
{
	{
		// Copy settings to audio-render-thread version
		FScopeLock ScopeLock(&SettingsCritSect);
		SettingsCopy = Settings; 
	}
	// This sets the 'bChanged' to true on related effect instances which will 
	// trigger a OnPresetChanged call on the audio render thread.
	Update(); 
} 

FSubmixEffectConvolutionReverbSettings USubmixEffectConvolutionReverbPreset::GetSettings() const
{ 
	FScopeLock ScopeLock(&SettingsCritSect);
	return SettingsCopy; 
} 

void USubmixEffectConvolutionReverbPreset::SetSettings(const FSubmixEffectConvolutionReverbSettings& InSettings)
{
	Settings = InSettings;

	if (nullptr != ImpulseResponse)
	{
		SetImpulseResponseSettings(ImpulseResponse);
	}

	UpdateSettings();
}

void USubmixEffectConvolutionReverbPreset::SetImpulseResponse(UAudioImpulseResponse* InImpulseResponse)
{
	ImpulseResponse = InImpulseResponse;
	SetImpulseResponseSettings(InImpulseResponse);
	RebuildConvolutionReverb();
}

void USubmixEffectConvolutionReverbPreset::RebuildConvolutionReverb()
{
	using namespace AudioConvReverbIntrinsics;
	using FEffectWeakPtr = TWeakPtr<FSoundEffectBase, ESPMode::ThreadSafe>;
	using FEffectSharedPtr = TSharedPtr<FSoundEffectBase, ESPMode::ThreadSafe>;
	using FSubmixEffectConvSharedPtr = TSharedPtr<FSubmixEffectConvolutionReverb, ESPMode::ThreadSafe>;

	// Iterator over effects and update the convolution Reverb->
	for (FEffectWeakPtr& InstanceWeakPtr : Instances)
	{
		FEffectSharedPtr InstanceSharedPtr = InstanceWeakPtr.Pin();
		if (InstanceSharedPtr.IsValid())
		{
			FSubmixEffectConvSharedPtr ConvEffectSharedPtr = StaticCastSharedPtr<FSubmixEffectConvolutionReverb>(InstanceSharedPtr);

			FVersionData VersionData = ConvEffectSharedPtr->UpdateConvolutionReverb(this);
			ConvEffectSharedPtr->RebuildConvolutionReverb();
		}
	}
}

#if WITH_EDITORONLY_DATA
void USubmixEffectConvolutionReverbPreset::BindToImpulseResponseObjectChange()
{
	if (nullptr != ImpulseResponse)
	{
		if (!DelegateHandles.Contains(ImpulseResponse))
		{
			FDelegateHandle Handle = ImpulseResponse->OnObjectPropertyChanged.AddUObject(this, &USubmixEffectConvolutionReverbPreset::PostEditChangeImpulseProperty);
			if (Handle.IsValid())
			{
				DelegateHandles.Add(ImpulseResponse, Handle);
			}
		}
	}
}

void USubmixEffectConvolutionReverbPreset::PreEditChange(FProperty* PropertyAboutToChange)
{
	Super::PreEditChange(PropertyAboutToChange);

	if (nullptr == PropertyAboutToChange)
	{
		return;
	}

	const FName ImpulseResponseFName = GET_MEMBER_NAME_CHECKED(USubmixEffectConvolutionReverbPreset, ImpulseResponse);

	if (PropertyAboutToChange->GetFName() == ImpulseResponseFName)
	{
		if (nullptr == ImpulseResponse)
		{
			return;
		}

		if (DelegateHandles.Contains(ImpulseResponse))
		{
			FDelegateHandle DelegateHandle = DelegateHandles[ImpulseResponse];

			if (DelegateHandle.IsValid())
			{
				ImpulseResponse->OnObjectPropertyChanged.Remove(DelegateHandle);
			}

			DelegateHandles.Remove(ImpulseResponse);
		}
	}
}



void USubmixEffectConvolutionReverbPreset::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName ImpulseResponseFName = GET_MEMBER_NAME_CHECKED(USubmixEffectConvolutionReverbPreset, ImpulseResponse);

	// If any of these properties are updated, the FConvolutionReverb object needs
	// to be rebuilt.
	const TArray<FName> ConvolutionReverbProperties(
		{
			ImpulseResponseFName,
			GET_MEMBER_NAME_CHECKED(USubmixEffectConvolutionReverbPreset, bEnableHardwareAcceleration),
			GET_MEMBER_NAME_CHECKED(USubmixEffectConvolutionReverbPreset, BlockSize),
			GET_MEMBER_NAME_CHECKED(FSubmixEffectConvolutionReverbSettings, bMixInputChannelFormatToImpulseResponseFormat),
			GET_MEMBER_NAME_CHECKED(FSubmixEffectConvolutionReverbSettings, bMixReverbOutputToOutputChannelFormat),
		}
	);

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		if (FProperty* PropertyThatChanged = PropertyChangedEvent.Property)
		{
			const FName& Name = PropertyThatChanged->GetFName();

			// Need to update impulse reponse bindings and normalization before rebuilding convolution reverb
			if (Name == ImpulseResponseFName)
			{
				BindToImpulseResponseObjectChange();
				
				if (nullptr != ImpulseResponse)
				{
					SetImpulseResponseSettings(ImpulseResponse);
				}
			}

			if (ConvolutionReverbProperties.Contains(Name))
			{
				RebuildConvolutionReverb();
			}
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void USubmixEffectConvolutionReverbPreset::PostEditChangeImpulseProperty(FPropertyChangedEvent& PropertyChangedEvent) 
{
	const FName NormalizationVolumeFName = GET_MEMBER_NAME_CHECKED(UAudioImpulseResponse, NormalizationVolumeDb);
	const FName TrueStereoFName = GET_MEMBER_NAME_CHECKED(UAudioImpulseResponse, bTrueStereo);

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		if (FProperty* PropertyThatChanged = PropertyChangedEvent.Property)
		{
			const FName& Name = PropertyThatChanged->GetFName();

			if (Name == NormalizationVolumeFName)
			{
				if (nullptr != ImpulseResponse)
				{
					Settings.NormalizationVolumeDb = ImpulseResponse->NormalizationVolumeDb;
					UpdateSettings();
				}
			}
			else if (Name == TrueStereoFName)
			{
				RebuildConvolutionReverb();
			}
		}
	}
}

#endif

void USubmixEffectConvolutionReverbPreset::SetImpulseResponseSettings(UAudioImpulseResponse* InImpulseResponse)
{
	if (nullptr != InImpulseResponse)
	{
		// Set this value, but do not call UpdateSettings(). UpdateSettings() is handled elsewhere.
		Settings.NormalizationVolumeDb = InImpulseResponse->NormalizationVolumeDb;
	}
}

void USubmixEffectConvolutionReverbPreset::UpdateDeprecatedProperties()
{
	if (Settings.SurroundRearChannelBleedAmount_DEPRECATED != 0.f)
	{
		Settings.SurroundRearChannelBleedDb = Audio::ConvertToDecibels(FMath::Abs(Settings.SurroundRearChannelBleedAmount_DEPRECATED));
		Settings.bInvertRearChannelBleedPhase = Settings.SurroundRearChannelBleedAmount_DEPRECATED < 0.f;

		Settings.SurroundRearChannelBleedAmount_DEPRECATED = 0.f;
		Modify();
	}

	if (nullptr != Settings.ImpulseResponse_DEPRECATED)
	{
		ImpulseResponse = Settings.ImpulseResponse_DEPRECATED;

		// Need to make a copy of existing samples and reinterleave 
		// The previous version had these sampels chunked by channel
		// like [[all channel 0 samples][all channel 1 samples][...][allchannel N samples]]
		//
		// They need to be in interleave format to work in this class.

		int32 NumChannels = Settings.ImpulseResponse_DEPRECATED->NumChannels;

		if (NumChannels > 1)
		{
			const TArray<float>& IRData = Settings.ImpulseResponse_DEPRECATED->IRData_DEPRECATED;
			int32 NumSamples = IRData.Num();

			if (NumSamples > 0)
			{
				ImpulseResponse->ImpulseResponse.Reset();
				ImpulseResponse->ImpulseResponse.AddUninitialized(NumSamples);

				int32 NumFrames = NumSamples / NumChannels;

				const float* InputSamples = IRData.GetData();
				float* OutputSamples = ImpulseResponse->ImpulseResponse.GetData();

				for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ChannelIndex++)
				{
					for (int32 FrameIndex = 0; FrameIndex < NumFrames; FrameIndex++)
					{
						OutputSamples[FrameIndex * NumChannels + ChannelIndex] = InputSamples[ChannelIndex * NumFrames + FrameIndex];
					}
				}
			}
		}
		else
		{
			// Do not need to do anything for zero or one channels.
			ImpulseResponse->ImpulseResponse = ImpulseResponse->IRData_DEPRECATED;
		}

		// Discard samples after they have been copied.
		Settings.ImpulseResponse_DEPRECATED->IRData_DEPRECATED = TArray<float>();

		Settings.ImpulseResponse_DEPRECATED = nullptr;

		Modify();
	}

	if (!Settings.AllowHardwareAcceleration_DEPRECATED)
	{
		bEnableHardwareAcceleration = false;

		Settings.AllowHardwareAcceleration_DEPRECATED = true;
		Modify();
	}
}

void USubmixEffectConvolutionReverbPreset::PostLoad()
{
	Super::PostLoad();

	// This handles previous version 
	UpdateDeprecatedProperties();

#if WITH_EDITORONLY_DATA
	// Bind to trigger new convolution algorithms when UAudioImpulseResponse object changes.
	BindToImpulseResponseObjectChange();
#endif

	SetImpulseResponseSettings(ImpulseResponse);
}

