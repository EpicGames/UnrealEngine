// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubmixEffects/SubmixEffectDelay.h"
#include "DSP/Delay.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SubmixEffectDelay)

const float FSubmixEffectDelay::MinLengthDelaySec = 0.4f;

FSubmixEffectDelay::FSubmixEffectDelay()
	: SampleRate(0.0f)
	, MaxDelayLineLength(10000.0f)
	, InterpolationTime(0.0f)
	, TargetDelayLineLength(5000.0f)
{
}

FSubmixEffectDelay::~FSubmixEffectDelay()
{
}

void FSubmixEffectDelay::Init(const FSoundEffectSubmixInitData& InData)
{
	SampleRate = InData.SampleRate;
	InterpolationInfo.Init(SampleRate);
}

void FSubmixEffectDelay::OnProcessAudio(const FSoundEffectSubmixInputData& InData, FSoundEffectSubmixOutputData& OutData)
{
	if (DelayLines.Num() != InData.NumChannels)
	{
		OnNumChannelsChanged(InData.NumChannels);
	}

	UpdateParameters();

	// Cache all pointers first to avoid bounds checks on dereferences:
	const float* InBuffer = InData.AudioBuffer->GetData();
	float* OutBuffer = OutData.AudioBuffer->GetData();

	Audio::FDelay* DelaysPtr = DelayLines.GetData();

	int32 NumDelays = DelayLines.Num();
	
	// If we have no taps to render, short circuit.
	if (!NumDelays)
	{
		return;
	}

	for (int32 OutputBufferIndex = 0; OutputBufferIndex < OutData.AudioBuffer->Num(); OutputBufferIndex += OutData.NumChannels)
	{
		for (int32 DelayIndex = 0; DelayIndex < NumDelays; DelayIndex++)
		{
			const int32 SampleIndex = OutputBufferIndex + DelayIndex;
			OutBuffer[SampleIndex] = DelaysPtr[DelayIndex].ProcessAudioSample(InBuffer[SampleIndex]);
			DelayLines[DelayIndex].SetDelayMsec(InterpolationInfo.GetNextValue());
		}
	}
}

void FSubmixEffectDelay::OnPresetChanged()
{
	GET_EFFECT_SETTINGS(SubmixEffectDelay);

	FSubmixEffectDelaySettings NewSettings;
	NewSettings = Settings;

	SetEffectParameters(NewSettings);
}

void FSubmixEffectDelay::SetEffectParameters(const FSubmixEffectDelaySettings& InTapEffectParameters)
{
	Params.SetParams(InTapEffectParameters);
}

void FSubmixEffectDelay::SetInterpolationTime(float Time)
{
	InterpolationTime = Time / 1000.0f;
	InterpolationInfo.SetValue(TargetDelayLineLength, InterpolationTime);
}

void FSubmixEffectDelay::SetDelayLineLength(float Length)
{
	TargetDelayLineLength = FMath::Clamp(Length, MinLengthDelaySec, MaxDelayLineLength);
	InterpolationInfo.SetValue(TargetDelayLineLength, InterpolationTime);
}

void FSubmixEffectDelay::UpdateParameters()
{

	FSubmixEffectDelaySettings NewSettings;

	if (Params.GetParams(&NewSettings))
	{
		Audio::FDelay* DelaysPtr = DelayLines.GetData();

		const float LastLength = MaxDelayLineLength;

		MaxDelayLineLength = FMath::Max(NewSettings.MaximumDelayLength, MinLengthDelaySec);
		InterpolationTime = NewSettings.InterpolationTime / 1000.0f;

		SetDelayLineLength(NewSettings.DelayLength);

		if (MaxDelayLineLength != LastLength)
		{
			for (int32 DelayIndex = 0; DelayIndex < DelayLines.Num(); DelayIndex++)
			{
				DelaysPtr[DelayIndex].Init(SampleRate, MaxDelayLineLength / 1000.0f);
			}
		}
	}
}

void FSubmixEffectDelay::OnNumChannelsChanged(int32 NumChannels)
{
	const int32 PriorNumDelayLines = DelayLines.Num();
	int32 NumAdditionalChannels = NumChannels - PriorNumDelayLines;

	DelayLines.SetNum(NumChannels);

	//If we have an additional amount of delay lines, initialize them here:
	if (NumAdditionalChannels > 0)
	{
		Audio::FDelay* DelayLinesPtr = DelayLines.GetData();

		for (int32 DelayIndex = PriorNumDelayLines; DelayIndex < DelayLines.Num(); DelayIndex++)
		{
			DelayLinesPtr[DelayIndex].Init(SampleRate, MaxDelayLineLength / 1000.0f);
		}
	}
}

void USubmixEffectDelayPreset::SetInterpolationTime(float Time)
{
	DynamicSettings.InterpolationTime = Time;

	// Dispatch to all effect instances:
	EffectCommand<FSubmixEffectDelay>([Time](FSubmixEffectDelay& TapDelay)
	{
		TapDelay.SetInterpolationTime(Time);
	});
}

void USubmixEffectDelayPreset::SetDelay(float Length)
{
	DynamicSettings.DelayLength = Length;

	// Dispatch to all effect instances:
	EffectCommand<FSubmixEffectDelay>([Length](FSubmixEffectDelay& TapDelay)
	{
		TapDelay.SetDelayLineLength(Length);
	});
}

void USubmixEffectDelayPreset::OnInit()
{
	// Copy the settings to our dynamic settings so we can modify
	DynamicSettings = Settings;
}

void USubmixEffectDelayPreset::SetSettings(const FSubmixEffectDelaySettings& InSettings)
{
	DynamicSettings = InSettings;

// 	USubmixEffectDelayStatics::SetDelayLength(DynamicSettings, DynamicSettings.DelayLength);
// 	USubmixEffectDelayStatics::SetMaximumDelayLength(DynamicSettings, DynamicSettings.MaximumDelayLength);

	UpdateSettings(DynamicSettings);
}

void USubmixEffectDelayPreset::SetDefaultSettings(const FSubmixEffectDelaySettings& InSettings)
{
	Settings = InSettings;

// 	USubmixEffectDelayStatics::SetDelayLength(Settings, Settings.DelayLength);
// 	USubmixEffectDelayStatics::SetMaximumDelayLength(Settings, Settings.MaximumDelayLength);

	DynamicSettings = Settings;
	UpdateSettings(DynamicSettings);

	MarkPackageDirty();
}

