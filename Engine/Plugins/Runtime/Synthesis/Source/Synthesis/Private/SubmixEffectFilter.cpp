// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubmixEffects/SubmixEffectFilter.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SubmixEffectFilter)


FSubmixEffectFilter::FSubmixEffectFilter()
	: SampleRate(0.0f)
	, CurrentFilter(nullptr)
	, FilterAlgorithm(ESubmixFilterAlgorithm::OnePole)
	, FilterType(ESubmixFilterType::LowPass)
	, FilterFrequency(0.0f)
	, FilterFrequencyMod(0.0f)
	, FilterQ(0.0f)
	, FilterQMod(0.0f)
	, NumChannels(0)
{
}

void FSubmixEffectFilter::Init(const FSoundEffectSubmixInitData& InData)
{
	SampleRate = InData.SampleRate;
	CurrentFilter = &OnePoleFilter;
	NumChannels = 2;

	InitFilter();

}

void FSubmixEffectFilter::InitFilter()
{
	OnePoleFilter.Init(SampleRate, NumChannels);
	StateVariableFilter.Init(SampleRate, NumChannels);
	LadderFilter.Init(SampleRate, NumChannels);

	// Reset all the things on the current filter
	CurrentFilter->SetFilterType((Audio::EFilter::Type)FilterType);
	CurrentFilter->SetFrequency(FilterFrequency);
	CurrentFilter->SetQ(FilterQ);
	CurrentFilter->SetFrequencyMod(FilterFrequencyMod);
	CurrentFilter->SetQMod(FilterQMod);
}

void FSubmixEffectFilter::OnProcessAudio(const FSoundEffectSubmixInputData& InData, FSoundEffectSubmixOutputData& OutData)
{
	CurrentFilter->Update();

	if (NumChannels != InData.NumChannels)
	{
		NumChannels = InData.NumChannels;
		InitFilter();
	}

	float* InAudioBuffer = InData.AudioBuffer->GetData();
	float* OutAudioBuffer = OutData.AudioBuffer->GetData();
	const int32 NumSamples = InData.AudioBuffer->Num();

	CurrentFilter->ProcessAudio(InAudioBuffer, NumSamples, OutAudioBuffer);
}

void FSubmixEffectFilter::OnPresetChanged()
{
	GET_EFFECT_SETTINGS(SubmixEffectFilter);

	FSubmixEffectFilterSettings NewSettings;
	NewSettings = Settings;

	if (NewSettings.FilterAlgorithm != FilterAlgorithm)
	{
		FilterFrequency = NewSettings.FilterFrequency;
		FilterType = NewSettings.FilterType;
		FilterQ = NewSettings.FilterQ;

		SetFilterAlgorithm(NewSettings.FilterAlgorithm);
	}
	else
	{
		SetFilterCutoffFrequency(NewSettings.FilterFrequency);
		SetFilterQ(NewSettings.FilterQ);
		SetFilterType(NewSettings.FilterType);
	}
}

void FSubmixEffectFilter::SetFilterType(ESubmixFilterType InType)
{
	if (FilterType != InType)
	{
		FilterType = InType;
		CurrentFilter->SetFilterType((Audio::EFilter::Type)FilterType);
	}
}

void FSubmixEffectFilter::SetFilterAlgorithm(ESubmixFilterAlgorithm InAlgorithm)
{
	if (InAlgorithm != FilterAlgorithm)
	{
		FilterAlgorithm = InAlgorithm;

		switch (FilterAlgorithm)
		{
			case ESubmixFilterAlgorithm::OnePole:
				CurrentFilter = &OnePoleFilter;
				break;

			case ESubmixFilterAlgorithm::StateVariable:
				CurrentFilter = &StateVariableFilter;
				break;

			case ESubmixFilterAlgorithm::Ladder:
				CurrentFilter = &LadderFilter;
				break;
		}

		CurrentFilter->SetFilterType((Audio::EFilter::Type)FilterType);
		CurrentFilter->SetFrequency(FilterFrequency);
		CurrentFilter->SetQ(FilterQ);
		CurrentFilter->SetFrequencyMod(FilterFrequencyMod);
		CurrentFilter->SetQMod(FilterQMod);
	}
}

void FSubmixEffectFilter::SetFilterCutoffFrequency(float InFrequency)
{
	// Ensure that we are always below the nyquist frequency:
	const float MaxCutoffFrequency = (SampleRate / 2.0f) * 0.99f;

	InFrequency = FMath::Clamp(InFrequency, 0.0f, MaxCutoffFrequency);

	if (!FMath::IsNearlyEqual(InFrequency, FilterFrequency))
	{
		FilterFrequency = InFrequency;
		CurrentFilter->SetFrequency(FilterFrequency);
	}
}

void FSubmixEffectFilter::SetFilterCutoffFrequencyMod(float InFrequency)
{
	if (!FMath::IsNearlyEqual(InFrequency, FilterFrequencyMod))
	{
		FilterFrequencyMod = InFrequency;
		CurrentFilter->SetFrequencyMod(FilterFrequencyMod);
	}
}

void FSubmixEffectFilter::SetFilterQ(float InQ)
{
	if (!FMath::IsNearlyEqual(InQ, FilterQ))
	{
		FilterQ = InQ;
		CurrentFilter->SetQ(FilterQ);
	}
}

void FSubmixEffectFilter::SetFilterQMod(float InQ)
{
	if (!FMath::IsNearlyEqual(InQ, FilterQMod))
	{
		FilterQMod = InQ;
		CurrentFilter->SetQMod(FilterQMod);
	}
}

void USubmixEffectFilterPreset::SetSettings(const FSubmixEffectFilterSettings& InSettings)
{
	UpdateSettings(InSettings);
}

void USubmixEffectFilterPreset::SetFilterType(ESubmixFilterType InType)
{
	EffectCommand<FSubmixEffectFilter>([InType](FSubmixEffectFilter& FilterEffect)
	{
		FilterEffect.SetFilterType(InType);
	});
}

void USubmixEffectFilterPreset::SetFilterAlgorithm(ESubmixFilterAlgorithm InAlgorithm)
{
	EffectCommand<FSubmixEffectFilter>([InAlgorithm](FSubmixEffectFilter& FilterEffect)
	{
		FilterEffect.SetFilterAlgorithm(InAlgorithm);
	});
}

void USubmixEffectFilterPreset::SetFilterCutoffFrequency(float InFrequency)
{
	EffectCommand<FSubmixEffectFilter>([InFrequency](FSubmixEffectFilter& FilterEffect)
	{
		FilterEffect.SetFilterCutoffFrequency(InFrequency);
	});
}

void USubmixEffectFilterPreset::SetFilterCutoffFrequencyMod(float InFrequency)
{
	EffectCommand<FSubmixEffectFilter>([InFrequency](FSubmixEffectFilter& FilterEffect)
	{
		FilterEffect.SetFilterCutoffFrequencyMod(InFrequency);
	});
}

void USubmixEffectFilterPreset::SetFilterQ(float InQ)
{
	EffectCommand<FSubmixEffectFilter>([InQ](FSubmixEffectFilter& FilterEffect)
	{
		FilterEffect.SetFilterQ(InQ);
	});
}

void USubmixEffectFilterPreset::SetFilterQMod(float InQ)
{
	EffectCommand<FSubmixEffectFilter>([InQ](FSubmixEffectFilter& FilterEffect)
	{
		FilterEffect.SetFilterQMod(InQ);
	});
}

