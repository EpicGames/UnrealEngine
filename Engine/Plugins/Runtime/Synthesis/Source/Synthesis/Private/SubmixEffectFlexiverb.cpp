// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubmixEffects/SubmixEffectFlexiverb.h"
#include "DSP/AllPassFilter.h" // IWYU pragma: keep
#include "DSP/OnePole.h" // IWYU pragma: keep

#include UE_INLINE_GENERATED_CPP_BY_NAME(SubmixEffectFlexiverb)


FSubmixEffectFlexiverb::FSubmixEffectFlexiverb()
	: bIsEnabled(false)
{
}

void FSubmixEffectFlexiverb::Init(const FSoundEffectSubmixInitData& InitData)
{
	Audio::FFlexiverbSettings NewSettings;
	Params.SetParams(NewSettings);

	Flexiverb.Init(InitData.SampleRate, NewSettings);
	bIsEnabled = true;
}

void FSubmixEffectFlexiverb::OnPresetChanged()
{
	GET_EFFECT_SETTINGS(SubmixEffectFlexiverb);
	
	Audio::FFlexiverbSettings NewSettings;

	NewSettings.Complexity = Settings.Complexity;
	NewSettings.RoomDampening = Settings.RoomDampening;
	NewSettings.DecayTime = Settings.DecayTime;
	NewSettings.PreDelay = Settings.PreDelay;

	SetEffectParameters(NewSettings);
}

void FSubmixEffectFlexiverb::SetEffectParameters(const Audio::FFlexiverbSettings& InReverbEffectParameters)
{
	Params.SetParams(InReverbEffectParameters);
}

void FSubmixEffectFlexiverb::OnProcessAudio(const FSoundEffectSubmixInputData& InData, FSoundEffectSubmixOutputData& OutData)
{
	UpdateParameters();

	const float* InputBuffer = InData.AudioBuffer->GetData();
	float* OutputBuffer = OutData.AudioBuffer->GetData();

	int32 InputBufferIndex = 0;
	for (int32 OutputBufferIndex = 0; OutputBufferIndex < OutData.AudioBuffer->Num(); OutputBufferIndex += OutData.NumChannels)
	{
		Flexiverb.ProcessAudioFrame(&(InputBuffer[InputBufferIndex]), InData.NumChannels, &(OutputBuffer[OutputBufferIndex]), OutData.NumChannels);
		InputBuffer += InData.NumChannels;
	}
}

void FSubmixEffectFlexiverb::UpdateParameters()
{
 	Audio::FFlexiverbSettings NewSettings;
 	if (Params.GetParams(&NewSettings))
 	{
 		Flexiverb.SetSettings(NewSettings);
 	}
}

void USubmixEffectFlexiverbPreset::SetSettings(const FSubmixEffectFlexiverbSettings& InSettings)
{
	UpdateSettings(InSettings);
}
