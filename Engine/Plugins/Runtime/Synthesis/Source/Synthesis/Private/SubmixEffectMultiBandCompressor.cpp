// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubmixEffects/SubmixEffectMultiBandCompressor.h"

#include "AudioBusSubsystem.h"
#include "AudioMixerDevice.h"
#include "AudioMixerSubmix.h"
#include "DSP/FloatArrayMath.h"
#include "Sound/AudioBus.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SubmixEffectMultiBandCompressor)

FSubmixEffectMultibandCompressor::FSubmixEffectMultibandCompressor()
{
	ScratchBuffer.AddUninitialized(FSubmixEffectMultibandCompressor::MaxBlockNumSamples);

	DeviceCreatedHandle = FAudioDeviceManagerDelegates::OnAudioDeviceCreated.AddRaw(this, &FSubmixEffectMultibandCompressor::OnDeviceCreated);
	DeviceDestroyedHandle = FAudioDeviceManagerDelegates::OnAudioDeviceDestroyed.AddRaw(this, &FSubmixEffectMultibandCompressor::OnDeviceDestroyed);
}

FSubmixEffectMultibandCompressor::~FSubmixEffectMultibandCompressor()
{
	ResetKey();

	FAudioDeviceManagerDelegates::OnAudioDeviceCreated.Remove(DeviceCreatedHandle);
	FAudioDeviceManagerDelegates::OnAudioDeviceDestroyed.Remove(DeviceDestroyedHandle);
}

Audio::FDeviceId FSubmixEffectMultibandCompressor::GetDeviceId() const
{
	return DeviceId;
}

void FSubmixEffectMultibandCompressor::Init(const FSoundEffectSubmixInitData& InitData)
{
	NumChannels = 0;
	SampleRate = InitData.SampleRate;

	MultiBandBuffer.Init(4, 2 * 1024);
	KeyMultiBandBuffer.Init(4, 2 * 1024);

	DeviceId = InitData.DeviceID;

	if (USubmixEffectMultibandCompressorPreset* ProcPreset = Cast<USubmixEffectMultibandCompressorPreset>(Preset.Get()))
	{
		switch (ProcPreset->Settings.KeySource)
		{
			case ESubmixEffectDynamicsKeySource::AudioBus:
			{
				if (UAudioBus* AudioBus = ProcPreset->Settings.ExternalAudioBus)
				{
					KeySource.Update(ESubmixEffectDynamicsKeySource::AudioBus, AudioBus->GetUniqueID(), static_cast<int32>(AudioBus->AudioBusChannels) + 1);
				}
			}
			break;

			case ESubmixEffectDynamicsKeySource::Submix:
			{
				if (USoundSubmix* Submix = ProcPreset->Settings.ExternalSubmix)
				{
					KeySource.Update(ESubmixEffectDynamicsKeySource::Submix, Submix->GetUniqueID());
				}
			}
			break;

			default:
			{
				// KeySource is this effect's submix/input, so do nothing
			}
			break;
		}
	}
}

void FSubmixEffectMultibandCompressor::ResetKey()
{
	KeySource.Reset();
}

Audio::FMixerDevice* FSubmixEffectMultibandCompressor::GetMixerDevice()
{
	if (FAudioDeviceManager* DeviceManager = FAudioDeviceManager::Get())
	{
		return static_cast<Audio::FMixerDevice*>(DeviceManager->GetAudioDeviceRaw(DeviceId));
	}

	return nullptr;
}

void FSubmixEffectMultibandCompressor::OnPresetChanged()
{
	GET_EFFECT_SETTINGS(SubmixEffectMultibandCompressor);

	if (Settings.Bands.Num() == 0 || NumChannels == 0)
	{
		return;
	}

	bool bNeedsReinit = (bInitialized == false
		|| Settings.Bands.Num() != PrevNumBands
		|| Settings.Bands.Num() - 1 != PrevCrossovers.Num()
		|| Settings.bFourPole != bPrevFourPole);

	// check if crossovers have changed
	// if so, update before potentially initializing
	bool bCrossoversChanged = false;
	for (int32 CrossoverId = 0; CrossoverId < PrevCrossovers.Num(); CrossoverId++)
	{
		float CrossoverFrequency = Settings.Bands[CrossoverId].CrossoverTopFrequency;
		if (CrossoverFrequency != PrevCrossovers[CrossoverId])
		{
			PrevCrossovers.Reset(Settings.Bands.Num());
			for (int32 BandId = 0; BandId < Settings.Bands.Num(); ++BandId)
			{
				PrevCrossovers.Add(Settings.Bands[BandId].CrossoverTopFrequency);
			}

			bCrossoversChanged = true;
			break;
		}
	}

	if (bNeedsReinit)
	{
		// only necessary when # of bands or filters is changed
		Initialize(Settings);
	}
	else if (bCrossoversChanged)
	{
		// lighter way to update crossovers if nothing else needs to be reinit
		BandSplitter.SetCrossovers(PrevCrossovers);
		KeyBandSplitter.SetCrossovers(PrevCrossovers);
	}

	bBypass = Settings.bBypass;

	Audio::EDynamicsProcessingMode::Type TypeToSet;
	switch (Settings.DynamicsProcessorType)
	{
	default:
	case ESubmixEffectDynamicsProcessorType::Compressor:
		TypeToSet = Audio::EDynamicsProcessingMode::Compressor;
		break;

	case ESubmixEffectDynamicsProcessorType::Limiter:
		TypeToSet = Audio::EDynamicsProcessingMode::Limiter;
		break;

	case ESubmixEffectDynamicsProcessorType::Expander:
		TypeToSet = Audio::EDynamicsProcessingMode::Expander;
		break;

	case ESubmixEffectDynamicsProcessorType::Gate:
		TypeToSet = Audio::EDynamicsProcessingMode::Gate;
		break;
	}

	Audio::EPeakMode::Type PeakModeToSet;
	switch (Settings.PeakMode)
	{
	default:
	case ESubmixEffectDynamicsPeakMode::MeanSquared:
		PeakModeToSet = Audio::EPeakMode::MeanSquared;
		break;

	case ESubmixEffectDynamicsPeakMode::RootMeanSquared:
		PeakModeToSet = Audio::EPeakMode::RootMeanSquared;
		break;

	case ESubmixEffectDynamicsPeakMode::Peak:
		PeakModeToSet = Audio::EPeakMode::Peak;
		break;
	}

	for (int32 BandId = 0; BandId < DynamicsProcessors.Num(); ++BandId)
	{
		Audio::FDynamicsProcessor& DynamicsProcessor = DynamicsProcessors[BandId];

		DynamicsProcessor.SetChannelLinkMode(static_cast<Audio::EDynamicsProcessorChannelLinkMode>(Settings.LinkMode));

		DynamicsProcessor.SetLookaheadMsec(Settings.LookAheadMsec);
		DynamicsProcessor.SetAnalogMode(Settings.bAnalogMode);

		DynamicsProcessor.SetAttackTime(Settings.Bands[BandId].AttackTimeMsec);
		DynamicsProcessor.SetReleaseTime(Settings.Bands[BandId].ReleaseTimeMsec);
		DynamicsProcessor.SetThreshold(Settings.Bands[BandId].ThresholdDb);
		DynamicsProcessor.SetRatio(Settings.Bands[BandId].Ratio);
		DynamicsProcessor.SetKneeBandwidth(Settings.Bands[BandId].KneeBandwidthDb);
		DynamicsProcessor.SetInputGain(Settings.Bands[BandId].InputGainDb);
		DynamicsProcessor.SetOutputGain(Settings.Bands[BandId].OutputGainDb);

		DynamicsProcessor.SetProcessingMode(TypeToSet);
		DynamicsProcessor.SetPeakMode(PeakModeToSet);

		// key audition will work fine if there's no external submix, and will simply bypass compression
		DynamicsProcessor.SetKeyAudition(Settings.bKeyAudition);
		DynamicsProcessor.SetKeyGain(Settings.KeyGainDb);

		UpdateKeyFromSettings(Settings);
	}
}

bool FSubmixEffectMultibandCompressor::UpdateKeySourcePatch()
{
	// Default (input as key) does not use source patch, so don't
	// continue checking or updating state.
	if (KeySource.GetType() == ESubmixEffectDynamicsKeySource::Default)
	{
		return false;
	}

	if (KeySource.Patch.IsValid())
	{
		return true;
	}

	switch (KeySource.GetType())
	{
		case ESubmixEffectDynamicsKeySource::AudioBus:
		{
			// Retrieving/mutating the MixerDevice is only safe during OnProcessAudio calls if
			// it is not called during Teardown.  The DynamicsProcessor should be Reset via
			// the OnDeviceDestroyed callback (prior to FAudioDevice::Teardown), so this call
			// should never be hit during Teardown.
			if (Audio::FMixerDevice* MixerDevice = GetMixerDevice())
			{
				const uint32 ObjectId = KeySource.GetObjectId();
				if (ObjectId != INDEX_NONE)
				{
					UAudioBusSubsystem* AudioBusSubsystem = MixerDevice->GetSubsystem<UAudioBusSubsystem>();
					check(AudioBusSubsystem);
					KeySource.Patch = AudioBusSubsystem->AddPatchOutputForAudioBus(ObjectId, MixerDevice->GetNumOutputFrames(), KeySource.GetNumChannels());
					if (KeySource.Patch.IsValid())
					{
						for (Audio::FDynamicsProcessor& DynamicsProcessor : DynamicsProcessors)
						{
							DynamicsProcessor.SetKeyNumChannels(KeySource.GetNumChannels());
						}

						return true;
					}
				}
			}
		}
		break;

		case ESubmixEffectDynamicsKeySource::Submix:
		{
			// Retrieving/mutating the MixerDevice is only safe during OnProcessAudio calls if
			// it is not called during Teardown.  The DynamicsProcessor should be Reset via
			// the OnDeviceDestroyed callback (prior to FAudioDevice::Teardown), so this call
			// should never be hit during Teardown.
			if (Audio::FMixerDevice* MixerDevice = GetMixerDevice())
			{
				const uint32 ObjectId = KeySource.GetObjectId();
				if (ObjectId != INDEX_NONE)
				{
					KeySource.Patch = MixerDevice->AddPatchForSubmix(ObjectId, 1.0f /* PatchGain */);
					if (KeySource.Patch.IsValid())
					{
						Audio::FMixerSubmixPtr SubmixPtr = MixerDevice->FindSubmixInstanceByObjectId(KeySource.GetObjectId());
						if (SubmixPtr.IsValid())
						{
							const int32 SubmixNumChannels = SubmixPtr->GetNumOutputChannels();
							KeySource.SetNumChannels(SubmixNumChannels);

							for (Audio::FDynamicsProcessor& DynamicsProcessor : DynamicsProcessors)
							{
								DynamicsProcessor.SetKeyNumChannels(SubmixNumChannels);
							}

							return true;
						}
					}
				}
			}
		}
		break;

		case ESubmixEffectDynamicsKeySource::Default:
		default:
		break;
	}

	return false;
}

void FSubmixEffectMultibandCompressor::Initialize(FSubmixEffectMultibandCompressorSettings& Settings)
{
	const int32 NumBands = Settings.Bands.Num();

	if(NumBands <= 0)
	{
		bInitialized = false;
		DynamicsProcessors.Reset();
		PrevCrossovers.Reset();
		return;
	}
	
	FrameSize = sizeof(float) * NumChannels;

	PrevCrossovers.Reset(NumBands - 1);
	for (int32 BandId = 0; BandId < NumBands - 1; BandId++)
	{
		PrevCrossovers.Add(Settings.Bands[BandId].CrossoverTopFrequency);
	}

	Audio::EFilterOrder CrossoverMode = Settings.bFourPole ? Audio::EFilterOrder::FourPole : Audio::EFilterOrder::TwoPole;

	BandSplitter.Init(NumChannels, SampleRate, CrossoverMode, true /*Phase Compensate*/, PrevCrossovers);
	KeyBandSplitter.Init(NumChannels, SampleRate, CrossoverMode, false /*Phase Compensate*/, PrevCrossovers);

	MultiBandBuffer.SetBands(NumBands);
	KeyMultiBandBuffer.SetBands(NumBands);

	DynamicsProcessors.Reset(NumBands);
	DynamicsProcessors.AddDefaulted(NumBands);

	for (int32 BandId = 0; BandId < NumBands; ++BandId)
	{
		DynamicsProcessors[BandId].Init(SampleRate, NumChannels);
	}

	PrevNumBands = Settings.Bands.Num();
	bPrevFourPole = Settings.bFourPole;
	bInitialized = true;
}

void FSubmixEffectMultibandCompressor::OnProcessAudio(const FSoundEffectSubmixInputData& InData, FSoundEffectSubmixOutputData& OutData)
{
	if (NumChannels != InData.NumChannels)
	{
		GET_EFFECT_SETTINGS(SubmixEffectMultibandCompressor);
		NumChannels = InData.NumChannels;

		Initialize(Settings);
		OnPresetChanged();
	}

	const int32 NumSamples = InData.NumFrames * NumChannels;

	const Audio::FAlignedFloatBuffer& InBuffer = *InData.AudioBuffer;
	Audio::FAlignedFloatBuffer& OutBuffer = *OutData.AudioBuffer;

	if (bBypass || bInitialized == false || DynamicsProcessors.IsEmpty())
	{
		//passthrough
		FMemory::Memcpy(OutBuffer.GetData(), InBuffer.GetData(), FrameSize * InData.NumFrames);
		return;
	}

	int32 NumKeyChannels = DynamicsProcessors[0].GetKeyNumChannels();
	int32 NumKeySamples = InData.NumFrames * NumKeyChannels;

	AudioExternal.Reset(NumKeySamples);

	// If set to default, enforce num key channels to always be number of input channels.
	// If either unset or KeySource was changed between frames back to 'Default', NumKeyChannels
	// could be stale or left as initialized number of scratch channels.
	if (KeySource.GetType() == ESubmixEffectDynamicsKeySource::Default)
	{
		if (InData.NumChannels != NumKeyChannels)
		{
			NumKeyChannels = InData.NumChannels;
			NumKeySamples = InData.NumFrames * NumKeyChannels;

			for (Audio::FDynamicsProcessor& DynamicsProcessor : DynamicsProcessors)
			{
				DynamicsProcessor.SetKeyNumChannels(NumKeyChannels);
			}
		}
	}
	else
	{
		AudioExternal.AddZeroed(NumKeySamples);
	}

	const bool bUseKey = UpdateKeySourcePatch();

	if (bUseKey)
	{
		KeySource.Patch->PopAudio(AudioExternal.GetData(), NumKeySamples, true /* bUseLatestAudio */);
	}

	if (InData.NumChannels != DynamicsProcessors[0].GetNumChannels())
	{
		for (Audio::FDynamicsProcessor& DynamicsProcessor : DynamicsProcessors)
		{
			DynamicsProcessor.SetNumChannels(InData.NumChannels);
		}
	}

	// Zero buffer so multiple bands can all sum to it
	FMemory::Memzero(OutBuffer.GetData(), FrameSize * InData.NumFrames);

	// If passed more samples than the max block size, process in blocks
	const int32 BlockSize = FMath::Min(MaxBlockNumSamples, NumSamples);

	if (BlockSize > MultiBandBuffer.NumSamples)
	{
		MultiBandBuffer.SetSamples(NumSamples);

		if (bUseKey)
		{
			KeyMultiBandBuffer.SetSamples(NumKeySamples);
		}
	}

	for (int32 SampleIdx = 0; SampleIdx < NumSamples; SampleIdx += BlockSize)
	{
		const float* InPtr = &InBuffer[SampleIdx];
		float* OutPtr = &OutBuffer[SampleIdx];
		const int32 NumBlockFrames = BlockSize / NumChannels;

		const float* KeyPtr = nullptr;
		
		if (bUseKey)
		{
			const int32 KeySampleIdx = (SampleIdx / NumChannels) * NumKeyChannels;
			ensure(KeySampleIdx < AudioExternal.Num());
			KeyPtr = &AudioExternal[KeySampleIdx];
		}

		ScratchBuffer.SetNumUninitialized(BlockSize, EAllowShrinking::No);

		BandSplitter.ProcessAudioBuffer(InPtr, MultiBandBuffer, NumBlockFrames);

		if (bUseKey)
		{
			KeyBandSplitter.ProcessAudioBuffer(KeyPtr, KeyMultiBandBuffer, NumBlockFrames);
		}
		else
		{
			// reset key gain to unity in case this has changed recently on another thread
			for (Audio::FDynamicsProcessor& DynamicsProcessor : DynamicsProcessors)
			{
				//DynamicsProcessor.SetKeyGain(0.0f);
			}
		}

		TArrayView<const float> ScratchBufferView(ScratchBuffer.GetData(), BlockSize);
		TArrayView<float> OutPtrView(OutPtr, BlockSize);

		for (int32 Band = 0; Band < DynamicsProcessors.Num(); ++Band)
		{
			DynamicsProcessors[Band].ProcessAudio(MultiBandBuffer[Band], BlockSize, ScratchBuffer.GetData(), bUseKey ? KeyMultiBandBuffer[Band] : nullptr);
			Audio::ArrayMixIn(ScratchBufferView, OutPtrView);
		}
	}
}

void FSubmixEffectMultibandCompressor::UpdateKeyFromSettings(const FSubmixEffectMultibandCompressorSettings& InSettings)
{
	uint32 ObjectId = INDEX_NONE;
	int32 SourceNumChannels = 0;
	switch (InSettings.KeySource)
	{
		case ESubmixEffectDynamicsKeySource::AudioBus:
		{
			if (InSettings.ExternalAudioBus)
			{
				ObjectId = InSettings.ExternalAudioBus->GetUniqueID();
				SourceNumChannels = static_cast<int32>(InSettings.ExternalAudioBus->AudioBusChannels) + 1;
			}
		}
		break;

		case ESubmixEffectDynamicsKeySource::Submix:
		{
			if (InSettings.ExternalSubmix)
			{
				ObjectId = InSettings.ExternalSubmix->GetUniqueID();
			}
		}
		break;

		default:
		{
		}
		break;
	}

	KeySource.Update(InSettings.KeySource, ObjectId, SourceNumChannels);
}

void FSubmixEffectMultibandCompressor::OnDeviceCreated(Audio::FDeviceId InDeviceId)
{
	if (InDeviceId == DeviceId)
	{
		GET_EFFECT_SETTINGS(SubmixEffectMultibandCompressor);
		UpdateKeyFromSettings(Settings);

		FAudioDeviceManagerDelegates::OnAudioDeviceCreated.Remove(DeviceCreatedHandle);
	}
}

void FSubmixEffectMultibandCompressor::OnDeviceDestroyed(Audio::FDeviceId InDeviceId)
{
	if (InDeviceId == DeviceId)
	{
		// Reset the key on device destruction to avoid reinitializing
		// it during FAudioDevice::Teardown via ProcessAudio.
		ResetKey();
		FAudioDeviceManagerDelegates::OnAudioDeviceDestroyed.Remove(DeviceDestroyedHandle);
	}
}

/**
 * Preset
 */
void USubmixEffectMultibandCompressorPreset::OnInit()
{
	switch (Settings.KeySource)
	{
		case ESubmixEffectDynamicsKeySource::AudioBus:
		{
			SetAudioBus(Settings.ExternalAudioBus);
		}
		break;

		case ESubmixEffectDynamicsKeySource::Submix:
		{
			SetExternalSubmix(Settings.ExternalSubmix);
		}
		break;

		default:
		break;
	}
}

#if WITH_EDITOR
void USubmixEffectMultibandCompressorPreset::PostEditChangeChainProperty(struct FPropertyChangedChainEvent& InChainEvent)
{
	if (InChainEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(FSubmixEffectMultibandCompressorSettings, KeySource))
	{
		switch (Settings.KeySource)
		{
			case ESubmixEffectDynamicsKeySource::AudioBus:
			{
				Settings.ExternalSubmix = nullptr;
			}
			break;

			case ESubmixEffectDynamicsKeySource::Submix:
			{
				Settings.ExternalAudioBus = nullptr;
			}
			break;

			case ESubmixEffectDynamicsKeySource::Default:
			default:
			{
				Settings.ExternalSubmix = nullptr;
				Settings.ExternalAudioBus = nullptr;
				static_assert(static_cast<int32>(ESubmixEffectDynamicsKeySource::Count) == 3, "Possible missing KeySource switch case coverage");
			}
			break;
		}
	}

	Super::PostEditChangeChainProperty(InChainEvent);
}
#endif // WITH_EDITOR

void USubmixEffectMultibandCompressorPreset::ResetKey()
{
	EffectCommand<FSubmixEffectMultibandCompressor>([](FSubmixEffectMultibandCompressor& Instance)
	{
		Instance.ResetKey();
	});
}

void USubmixEffectMultibandCompressorPreset::SetAudioBus(UAudioBus* InAudioBus)
{
	int32 BusChannels = 0;
	if (InAudioBus)
	{
		BusChannels = static_cast<int32>(InAudioBus->AudioBusChannels) + 1;
		SetKey(ESubmixEffectDynamicsKeySource::AudioBus, InAudioBus, BusChannels);
	}
	else
	{
		ResetKey();
	}
}

void USubmixEffectMultibandCompressorPreset::SetExternalSubmix(USoundSubmix* InSubmix)
{
	if (InSubmix)
	{
		SetKey(ESubmixEffectDynamicsKeySource::Submix, InSubmix);
	}
	else
	{
		ResetKey();
	}
}

void USubmixEffectMultibandCompressorPreset::SetKey(ESubmixEffectDynamicsKeySource InKeySource, UObject* InObject, int32 InNumChannels)
{
	if (InObject)
	{
		EffectCommand<FSubmixEffectMultibandCompressor>([this, ObjectId = InObject->GetUniqueID(), InKeySource, InNumChannels](FSubmixEffectMultibandCompressor& Instance)
		{
			Instance.KeySource.Update(InKeySource, ObjectId, InNumChannels);
		});
	}
}

void USubmixEffectMultibandCompressorPreset::SetSettings(const FSubmixEffectMultibandCompressorSettings& InSettings)
{
	UpdateSettings(InSettings);

	IterateEffects<FSubmixEffectMultibandCompressor>([&](FSubmixEffectMultibandCompressor& Instance)
	{
		Instance.UpdateKeyFromSettings(InSettings);
	});
}

