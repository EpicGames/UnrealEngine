// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubmixEffects/SubmixEffectStereoDelay.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SubmixEffectStereoDelay)


FSubmixEffectStereoDelay::FSubmixEffectStereoDelay()
{
}

FSubmixEffectStereoDelay::~FSubmixEffectStereoDelay()
{
}

void FSubmixEffectStereoDelay::Init(const FSoundEffectSubmixInitData& InData)
{
	DelayStereo.Init(InData.SampleRate, 2);
}

void FSubmixEffectStereoDelay::OnProcessAudio(const FSoundEffectSubmixInputData& InData, FSoundEffectSubmixOutputData& OutData)
{
	if (InData.NumChannels > 2)
	{
		// Push audio to scratch stereo buffer
		
		// Mem-copy the input to the output
		FMemory::Memcpy(OutData.AudioBuffer->GetData(), InData.AudioBuffer->GetData(), InData.NumFrames * InData.NumChannels * sizeof(float));

		ScratchStereoBuffer.Reset();
		ScratchStereoBuffer.AddUninitialized(InData.NumFrames * 2);

		float* InputBufferPtr = InData.AudioBuffer->GetData();

		// Get the stereo data from the input buffer
		for (int32 Frame = 0; Frame < InData.NumFrames; ++Frame)
		{
			for (int32 Channel = 0; Channel < 2; ++Channel)
			{
				int32 OutputSampleIndex = 2 * Frame + Channel;
				int32 InputSamplesIndex = InData.NumChannels * Frame + Channel;
				ScratchStereoBuffer[OutputSampleIndex] = InputBufferPtr[InputSamplesIndex];
			}
		}

		DelayStereo.ProcessAudio(ScratchStereoBuffer.GetData(), ScratchStereoBuffer.Num(), ScratchStereoBuffer.GetData());

		float* OutputBufferPtr = OutData.AudioBuffer->GetData();

		// Get the stereo data from the input buffer
		for (int32 Frame = 0; Frame < InData.NumFrames; ++Frame)
		{
			for (int32 Channel = 0; Channel < 2; ++Channel)
			{
				int32 OutputSampleIndex = 2 * Frame + Channel;
				int32 InputSamplesIndex = InData.NumChannels * Frame + Channel;
				OutputBufferPtr[InputSamplesIndex] = ScratchStereoBuffer[OutputSampleIndex];
			}
		}
	}
	else
	{
		float* InputBufferPtr = InData.AudioBuffer->GetData();
		float* OutputBufferPtr = OutData.AudioBuffer->GetData();
		int32 NumSamples = InData.NumChannels * InData.NumFrames;
		DelayStereo.ProcessAudio(InputBufferPtr, NumSamples, OutputBufferPtr);
	}
}

void FSubmixEffectStereoDelay::OnPresetChanged()
{
	GET_EFFECT_SETTINGS(SubmixEffectStereoDelay);

	DelayStereo.SetDelayTimeMsec(Settings.DelayTimeMsec);
	DelayStereo.SetFeedback(Settings.Feedback);
	DelayStereo.SetWetLevel(Settings.WetLevel);
	DelayStereo.SetDelayRatio(Settings.DelayRatio);
	DelayStereo.SetMode((Audio::EStereoDelayMode::Type)Settings.DelayMode);
	DelayStereo.SetDryLevel(Settings.DryLevel);
	DelayStereo.SetFilterEnabled(Settings.bFilterEnabled);
	DelayStereo.SetFilterSettings((Audio::EBiquadFilter::Type)Settings.FilterType, Settings.FilterFrequency, Settings.FilterQ);
}

void USubmixEffectStereoDelayPreset::OnInit()
{
}

void USubmixEffectStereoDelayPreset::SetSettings(const FSubmixEffectStereoDelaySettings& InSettings)
{
	UpdateSettings(InSettings);
}



