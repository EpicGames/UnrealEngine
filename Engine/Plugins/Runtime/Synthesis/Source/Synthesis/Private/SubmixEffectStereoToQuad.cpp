// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubmixEffects/SubmixEffectStereoToQuad.h"
#include "DSP/Dsp.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SubmixEffectStereoToQuad)


FSubmixEffectStereoToQuad::FSubmixEffectStereoToQuad()
{
}

FSubmixEffectStereoToQuad::~FSubmixEffectStereoToQuad()
{
}

void FSubmixEffectStereoToQuad::Init(const FSoundEffectSubmixInitData& InData)
{
}

void FSubmixEffectStereoToQuad::OnProcessAudio(const FSoundEffectSubmixInputData& InData, FSoundEffectSubmixOutputData& OutData)
{
	// Copy inputs to outputs
	FMemory::Memcpy(OutData.AudioBuffer->GetData(), InData.AudioBuffer->GetData(), InData.NumFrames * InData.NumChannels * sizeof(float));

	if (InData.NumChannels > 2)
	{
		float* InputBufferPtr = InData.AudioBuffer->GetData();
		float* OutputBufferPtr = OutData.AudioBuffer->GetData();

		// Channel offset in SMPTE channel order for side left and side right
		int32 ChannelOffset = 0;

		// Quad channels
		if (InData.NumChannels == 4)
		{
			ChannelOffset = 2;
		}
		// 5.1 channels
		else if (InData.NumChannels == 6)
		{
			ChannelOffset = 4;
		}
		// 7.1 channels
		else if (InData.NumChannels == 8)
		{
			ChannelOffset = 6;
		}
		else
		{
			// Not supported
			return;
		}

		if (CurrentSettings.bFlipChannels)
		{
			// If we're not flipping the left and right channels:
			for (int32 Frame = 0; Frame < InData.NumFrames; ++Frame)
			{
				// OutSideLeft
				int32 LeftChannelInputIndex = Frame * InData.NumChannels;
				int32 SideLeftChannelOutputIndex = Frame * InData.NumChannels + ChannelOffset;

				// Swap the channel order when moving to the output channels
				OutputBufferPtr[SideLeftChannelOutputIndex + 1] += LinearGain * InputBufferPtr[LeftChannelInputIndex];
				OutputBufferPtr[SideLeftChannelOutputIndex] += LinearGain * InputBufferPtr[LeftChannelInputIndex + 1];
			}
		}
		else
		{
			// If we're not flipping the left and right channels:
			for (int32 Frame = 0; Frame < InData.NumFrames; ++Frame)
			{
				// OutSideLeft
				int32 LeftChannelInputIndex = Frame * InData.NumChannels;
				int32 SideLeftChannelOutputIndex = Frame * InData.NumChannels + ChannelOffset;

				// For regular pass through (left channel to left surround, right channel to right surround)
				OutputBufferPtr[SideLeftChannelOutputIndex] += LinearGain * InputBufferPtr[LeftChannelInputIndex];
				OutputBufferPtr[SideLeftChannelOutputIndex + 1] += LinearGain * InputBufferPtr[LeftChannelInputIndex + 1];
			}
		}
	}
}

void FSubmixEffectStereoToQuad::OnPresetChanged()
{
 	GET_EFFECT_SETTINGS(SubmixEffectStereoToQuad);

	CurrentSettings = Settings;

	// Do the decibels to linear conversion here
	LinearGain = Audio::ConvertToLinear(CurrentSettings.RearChannelGain);
}

void USubmixEffectStereoToQuadPreset::OnInit()
{
}

void USubmixEffectStereoToQuadPreset::SetSettings(const FSubmixEffectStereoToQuadSettings& InSettings)
{
	UpdateSettings(InSettings);
}



