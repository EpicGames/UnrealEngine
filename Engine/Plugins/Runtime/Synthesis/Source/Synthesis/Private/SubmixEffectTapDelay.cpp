// Copyright Epic Games, Inc. All Rights Reserved.
#include "SubmixEffects/SubmixEffectTapDelay.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SubmixEffectTapDelay)

DEFINE_LOG_CATEGORY(LogTapDelay);

namespace TapDelayUtils
{
	void GetMultichannelPan(const float& Input, const float& Angle, const float& Gain, const int32& NumChannels, float* DestinationFrame)
	{
		const float InputWithGain = Input * Gain;

		if (NumChannels == 2)
		{
			float LeftGain;
			float RightGain;
			Audio::GetStereoPan(Angle / 90.0f, LeftGain, RightGain);

			DestinationFrame[0] += LeftGain * InputWithGain;
			DestinationFrame[1] += RightGain * InputWithGain;
		}
		else
		{
			float QuadChannelMap[4] = { 0.0f, 0.0f, 0.0f, 0.0f };

			// We will spatialize using quad panning
			// Note input angle results in 0.0 being behind you
			// We offset the value to align the value so that 0.0 is "front left" i.e. index 0
			// We use the modulo to make sure it wraps to be between 0.0 and 1.0
			// We're also (for simplicity) assuming isotropic distribution of the quad speaker map.
			const float NormalizedAngle = FMath::Fmod((Angle + 180.0f) / 360.0f + 0.625f, 1.0f);

			const float ChannelFraction = 4 * NormalizedAngle;
			const int32 Channel0 = FMath::FloorToInt(ChannelFraction);
			const int32 Channel1 = (Channel0 + 1) % 4;
			const float ChannelAlpha = (ChannelFraction - Channel0) * 2.0f - 1.0f;

			Audio::GetStereoPan(ChannelAlpha, QuadChannelMap[Channel0], QuadChannelMap[Channel1]);

			// Now map to the specific channel outputs
			if (NumChannels == 6 || NumChannels == 8)
			{
				// Specifically skipping LFE, Center channel, and side channels
				DestinationFrame[0] += QuadChannelMap[0] * InputWithGain;
				DestinationFrame[1] += QuadChannelMap[1] * InputWithGain;
				DestinationFrame[5] += QuadChannelMap[2] * InputWithGain;
				DestinationFrame[4] += QuadChannelMap[3] * InputWithGain;
			}
			else if (NumChannels >= 4)
			{
				// only really supporting quad, but weird channel configs will do something
				for (int32 i = 0; i < 4; ++i)
				{
					DestinationFrame[i] += QuadChannelMap[i] * InputWithGain;
				}
			}
		}
	}

	// Static id to get unique tap Ids
	static int32 IdCount = 0;
} // namespace TapDelayUtils

FTapDelayInfo::FTapDelayInfo()
	: TapLineMode(ETapLineMode::Panning)
	, DelayLength(1000.0f)
	, Gain(-3.0f)
	, OutputChannel(0)
	, PanInDegrees(0.0f)
	, TapId(TapDelayUtils::IdCount++)
{
}

FSubmixEffectTapDelay::FSubmixEffectTapDelay()
	: SampleRate(0.0f)
	, MaxDelayLineLength(10000.0f)
	, TapIncrementsRemaining(0)
{
}

void FSubmixEffectTapDelay::Init(const FSoundEffectSubmixInitData& InData)
{
	SampleRate = InData.SampleRate;
	DelayLine.Init(SampleRate, MaxDelayLineLength / 1000.0f);
}

void FSubmixEffectTapDelay::OnProcessAudio(const FSoundEffectSubmixInputData& InData, FSoundEffectSubmixOutputData& OutData)
{
	UpdateParameters();
	check(InData.NumChannels == OutData.NumChannels);

	// Cache all pointers first to avoid bounds checks on dereferences:
	const float* InBuffer = InData.AudioBuffer->GetData();
	float* OutBuffer = OutData.AudioBuffer->GetData();

	const FTapDelayInfo* TapsPointer = TargetTaps.GetData();
	FTapDelayInterpolationInfo* CurrentTapsPtr = CurrentTaps.GetData();
	const int32 NumTaps = TargetTaps.Num();

	// If we have no taps to render, short circuit.
	if (!NumTaps)
	{
		return;
	}

	int32 InputBufferIndex = 0;
	for (int32 OutputBufferIndex = 0; OutputBufferIndex < OutData.AudioBuffer->Num(); OutputBufferIndex += OutData.NumChannels)
	{
		// Sum into this sample from each tap.
		for (int32 TapIndex = 0; TapIndex < NumTaps; TapIndex++)
		{
			const FTapDelayInfo& TargetTap = TapsPointer[TapIndex];
			FTapDelayInterpolationInfo& CurrentTap = CurrentTapsPtr[TapIndex];

			if (TargetTap.TapLineMode == ETapLineMode::Panning || TargetTap.TapLineMode == ETapLineMode::Disabled)
			{
				TapDelayUtils::GetMultichannelPan(DelayLine.ReadDelayAt(CurrentTap.GetLengthValue()), TargetTap.PanInDegrees, CurrentTap.GetGainValue(), OutData.NumChannels, &(OutBuffer[OutputBufferIndex]));
			}
			else
			{
				const int32 BufferIndex = OutputBufferIndex + (TargetTap.OutputChannel % OutData.NumChannels);
				OutBuffer[BufferIndex] += DelayLine.ReadDelayAt(CurrentTap.GetLengthValue()) * CurrentTap.GetGainValue();
			}
		}

		// Finally, write our input into the delay line.
		float Input = 0.0f;
		for (int32 Channel = 0; Channel < InData.NumChannels; Channel++)
		{
			Input += (&InBuffer[InputBufferIndex])[Channel];
		}
		
		DelayLine.WriteDelayAndInc(Input);
		InputBufferIndex += InData.NumChannels;
	}
}

void FSubmixEffectTapDelay::OnPresetChanged()
{
	GET_EFFECT_SETTINGS(SubmixEffectTapDelay);

	FSubmixEffectTapDelaySettings NewSettings;
	NewSettings = Settings;

	SetEffectParameters(NewSettings);
}

void FSubmixEffectTapDelay::SetEffectParameters(const FSubmixEffectTapDelaySettings& InTapEffectParameters)
{
	Params.SetParams(InTapEffectParameters);
}

void FSubmixEffectTapDelay::AddTap(int32 TapId)
{
	FTapDelayInfo Info = FTapDelayInfo();
	Info.TapId = TapId;
	
	
	TargetTaps.Add(Info);

	int32 TapIndex = CurrentTaps.Add(FTapDelayInterpolationInfo());
	CurrentTaps[TapIndex].Init(SampleRate);

	bSettingsModified = true;
}

void FSubmixEffectTapDelay::RemoveTap(int32 TapId)
{
	for (int32 TapIndex = 0; TapIndex < TargetTaps.Num(); TapIndex++)
	{
		if (TargetTaps[TapIndex].TapId == TapId)
		{
			TargetTaps.RemoveAtSwap(TapIndex, EAllowShrinking::Yes);
			CurrentTaps.RemoveAtSwap(TapIndex, EAllowShrinking::Yes);
			bSettingsModified = true;
		}
	}
}

void FSubmixEffectTapDelay::SetTap(int32 TapId, const FTapDelayInfo& DelayInfo)
{
	for (FTapDelayInfo& TapInfo : TargetTaps)
	{
		if (TapInfo.TapId == TapId)
		{
			TapInfo.TapLineMode = DelayInfo.TapLineMode;
			TapInfo.DelayLength = FMath::Clamp(DelayInfo.DelayLength, 0.1f, MaxDelayLineLength);
			TapInfo.PanInDegrees = DelayInfo.PanInDegrees;
			TapInfo.OutputChannel = DelayInfo.OutputChannel;

			// cache the tap's gain as a linear value:
			TapInfo.Gain = Audio::ConvertToLinear(DelayInfo.Gain);
			bSettingsModified = true;
			break;
		}
	}
}

void FSubmixEffectTapDelay::SetInterpolationTime(float Time)
{
	InterpolationTime = Time / 1000.0f;
	bSettingsModified = true;
}

void FSubmixEffectTapDelay::UpdateParameters()
{
	FSubmixEffectTapDelaySettings NewSettings;

	if (Params.GetParams(&NewSettings))
	{
		// If we have a new max delay line length, reallocate the delay line
		if (MaxDelayLineLength != NewSettings.MaximumDelayLength)
		{
			DelayLine.Init(SampleRate, NewSettings.MaximumDelayLength / 1000.0f);
			MaxDelayLineLength = NewSettings.MaximumDelayLength;
		}

		InterpolationTime = NewSettings.InterpolationTime / 1000.0f;

		// If we're being fed an empty list of taps, short circuit
		if (NewSettings.Taps.Num() == 0)
		{
			TargetTaps.Reset();
			CurrentTaps.Reset();
			return;
		}

		TargetTaps = NewSettings.Taps;

		// Update the number of current taps based on the number of target taps:
		const int32 CurrentNumTaps = CurrentTaps.Num();
		const int32 InterpolationsToAdd = TargetTaps.Num() - CurrentNumTaps;
		CurrentTaps.SetNum(TargetTaps.Num());

		// If there are any new current taps, initialize them:
		if (InterpolationsToAdd > 0)
		{
			for (int32 TapIndex = CurrentNumTaps; TapIndex < CurrentTaps.Num(); TapIndex++)
			{
				CurrentTaps[TapIndex].Init(SampleRate);
			}
		}

		// Cache all gain values as linear:
		for (int32 TapIndex = 0; TapIndex < TargetTaps.Num(); TapIndex++)
		{
			TargetTaps[TapIndex].Gain = Audio::ConvertToLinear(TargetTaps[TapIndex].Gain);
		}

		bSettingsModified = true;
	}


	if (bSettingsModified)
	{
		UpdateInterpolations();
		bSettingsModified = false;
	}
}

void FSubmixEffectTapDelay::UpdateInterpolations()
{
	// for each tap, pre-calculate our interpolation values
	for (int32 TapIndex = 0; TapIndex < CurrentTaps.Num(); TapIndex++)
	{
		FTapDelayInfo& TargetTap = TargetTaps[TapIndex];
		FTapDelayInterpolationInfo& CurrentTap = CurrentTaps[TapIndex];

		if (TargetTap.TapLineMode == ETapLineMode::Disabled)
		{
			// Do not change the delay line length and set target gain to 0.
			TargetTap.Gain = 0.0f;
		}
		else
		{
			// Clamp the delay line length to the maximum we can read, and calculate how much we should interpolate each sample.
			const float ClampedTarget = FMath::Clamp(TargetTap.DelayLength, 0.1f, MaxDelayLineLength);
			CurrentTap.SetLengthValue(ClampedTarget, InterpolationTime);
		}

		CurrentTap.SetGainValue(TargetTap.Gain, InterpolationTime);
	}
}

void USubmixEffectTapDelayPreset::SetInterpolationTime(float Time)
{
	DynamicSettings.InterpolationTime = Time;

	// Dispatch the tap removal to all effect instances:
	EffectCommand<FSubmixEffectTapDelay>([Time](FSubmixEffectTapDelay& TapDelay)
	{
		TapDelay.SetInterpolationTime(Time);
	});
}

void USubmixEffectTapDelayPreset::OnInit()
{
	// Copy the settings to our dynamic settings so we can modify
	DynamicSettings = Settings;
}

void USubmixEffectTapDelayPreset::SetSettings(const FSubmixEffectTapDelaySettings& InSettings)
{
	DynamicSettings = InSettings;
	UpdateSettings(InSettings);
}

void USubmixEffectTapDelayPreset::AddTap(int32& OutTapId)
{
	FTapDelayInfo TapInfo = FTapDelayInfo();

	OutTapId = TapInfo.TapId;

	DynamicSettings.Taps.Add(TapInfo);

	// Dispatch the added tap to all effect instances:
	EffectCommand<FSubmixEffectTapDelay>([OutTapId](FSubmixEffectTapDelay& TapDelay)
	{
		TapDelay.AddTap(OutTapId);
	});
}

void USubmixEffectTapDelayPreset::RemoveTap(int32 TapId)
{
	for (int32 TapIndex = 0; TapIndex < DynamicSettings.Taps.Num(); TapIndex++)
	{
		if (DynamicSettings.Taps[TapIndex].TapId == TapId)
		{
			DynamicSettings.Taps.RemoveAtSwap(TapIndex);

			// Dispatch the tap removal to all effect instances:
			EffectCommand<FSubmixEffectTapDelay>([TapId](FSubmixEffectTapDelay& TapDelay)
			{
				TapDelay.RemoveTap(TapId);
			});

			return;
		}
	}

	// if we got here, we couldn't find the tap ID.
	UE_LOGF(LogTapDelay, Warning, "Tried to remove Invalid Tap ID %d!", TapId);
	
}

void USubmixEffectTapDelayPreset::SetTap(int32 TapId, const FTapDelayInfo& TapInfo)
{
	for (FTapDelayInfo& DelayInfo : DynamicSettings.Taps)
	{
		if (DelayInfo.TapId == TapId)
		{
			// Copy over the tap delay info
			DelayInfo = TapInfo;

			// Restore the tap id to what it was since it just got stomped
			DelayInfo.TapId = TapId;

			// Dispatch the new tap to all effect instances:
			const FTapDelayInfo PassedTapInfo = TapInfo;
			EffectCommand<FSubmixEffectTapDelay>([TapId, PassedTapInfo](FSubmixEffectTapDelay& TapDelay)
			{
				TapDelay.SetTap(TapId, PassedTapInfo);
			});

			return;
		}
	}

	// if we got here, we couldn't find the tap ID.
	UE_LOGF(LogTapDelay, Warning, "Tried to set Invalid Tap ID %d!", TapId);
}

void USubmixEffectTapDelayPreset::GetTap(int32 TapId, FTapDelayInfo& TapInfo)
{
	for (FTapDelayInfo& Tap : DynamicSettings.Taps)
	{
		if (Tap.TapId == TapId)
		{
			TapInfo = Tap;
			return;
		}
	}

	// if we got here, we couldn't find the tap ID.
	UE_LOGF(LogTapDelay, Warning, "Tried to get Invalid Tap ID %d!", TapId);
}

void USubmixEffectTapDelayPreset::GetTapIds(TArray<int32>& TapIds)
{
	for (FTapDelayInfo& DelayInfo : DynamicSettings.Taps)
	{
		TapIds.Add(DelayInfo.TapId);
	}
}

void FTapDelayInterpolationInfo::Init(float SampleRate)
{
	LengthParam.Init(SampleRate);
	GainParam.Init(SampleRate);
}

void FTapDelayInterpolationInfo::SetGainValue(float Value, float InterpolationTime)
{
	GainParam.SetValue(Value, InterpolationTime);
}

float FTapDelayInterpolationInfo::GetGainValue()
{
	return GainParam.GetNextValue();
}

void FTapDelayInterpolationInfo::SetLengthValue(float Value, float InterpolationTime)
{
	LengthParam.SetValue(Value, InterpolationTime);
}

float FTapDelayInterpolationInfo::GetLengthValue()
{
	return LengthParam.GetNextValue();
}
