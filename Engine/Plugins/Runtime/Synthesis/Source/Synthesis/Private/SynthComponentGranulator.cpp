// Copyright Epic Games, Inc. All Rights Reserved.

#include "SynthComponents/SynthComponentGranulator.h"
#include "AudioDevice.h"
#include "Sound/SoundSourceBusSend.h"
#include "Sound/SoundSubmixSend.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SynthComponentGranulator)

UGranularSynth::UGranularSynth(const FObjectInitializer& ObjInitializer)
	: Super(ObjInitializer)
	, bIsLoaded(false)
	, bRegistered(false)
{
	PrimaryComponentTick.bCanEverTick = true;
}

UGranularSynth::~UGranularSynth()
{

}

bool UGranularSynth::Init(int32& SampleRate)
{
	NumChannels = 2;
	return true;
}

int32 UGranularSynth::OnGenerateAudio(float* OutAudio, int32 NumSamples)
{
	const int32 NumFrames = NumSamples / NumChannels;
	GranularSynth.Generate(OutAudio, NumFrames);
	return NumSamples;
}

void UGranularSynth::OnRegister()
{
	Super::OnRegister();

	if (!bRegistered)
	{
		bRegistered = true;
		SetComponentTickEnabled(true);
		RegisterComponent();

		if (FAudioDevice* AudioDevice = GetAudioDevice())
		{
			GranularSynth.Init(AudioDevice->GetSampleRate(), 500);
		}
	}
}

void UGranularSynth::OnUnregister()
{
	Super::OnUnregister();
}

void UGranularSynth::SetAttackTime(const float AttackTimeMsec)
{
	SynthCommand([this, AttackTimeMsec]()
	{
		GranularSynth.SetAttackTime(AttackTimeMsec);
	});
}

void UGranularSynth::SetDecayTime(const float DecayTimeMsec)
{
	SynthCommand([this, DecayTimeMsec]()
	{
		GranularSynth.SetAttackTime(DecayTimeMsec);
	});
}

void UGranularSynth::SetSustainGain(const float SustainGain)
{
	SynthCommand([this, SustainGain]()
	{
		GranularSynth.SetSustainGain(SustainGain);
	});
}

void UGranularSynth::SetReleaseTimeMsec(const float ReleaseTimeMsec)
{
	SynthCommand([this, ReleaseTimeMsec]()
	{
		GranularSynth.SetReleaseTime(ReleaseTimeMsec);
	});
}

void UGranularSynth::NoteOn(const float Note, const int32 Velocity, const float Duration)
{
	SynthCommand([this, Note, Velocity, Duration]()
	{
		GranularSynth.NoteOn(Note, Velocity, Duration);
	});
}

void UGranularSynth::NoteOff(const float Note, const bool bKill)
{
	SynthCommand([this, Note, bKill]()
	{
		GranularSynth.NoteOff(Note, bKill);
	});
}

void UGranularSynth::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction)
{
	SoundWaveLoader.Update();
}

void UGranularSynth::SetSoundWave(USoundWave* InSoundWave)
{
	if (InSoundWave != GranulatedSoundWave)
	{
		GranulatedSoundWave = InSoundWave;
		bIsLoaded = false;

		if (InSoundWave && InSoundWave->GetLoadingBehavior(/*bCheckSoundClasses*/ false) == ESoundWaveLoadingBehavior::ForceInline)
		{
			TFunction<void(const USoundWave * SoundWave, const Audio::FSampleBuffer & SampleBuffer)> OnLoaded
				= [this](const USoundWave * SoundWave, const Audio::FSampleBuffer & SampleBuffer)
			{
				if (SoundWave == GranulatedSoundWave)
				{
					SynthCommand([this, SampleBuffer]()
					{
						GranularSynth.LoadSampleBuffer(SampleBuffer);
					});

					bIsLoaded = true;
				}
			};

			SoundWaveLoader.LoadSoundWave(InSoundWave, MoveTemp(OnLoaded));
		}
	}
}

void UGranularSynth::SetGrainsPerSecond(const float GrainsPerSecond)
{
	SynthCommand([this, GrainsPerSecond]()
	{
		GranularSynth.SetGrainsPerSecond(GrainsPerSecond);
	});
}

void UGranularSynth::SetGrainProbability(const float InGrainProbability)
{
	SynthCommand([this, InGrainProbability]()
	{
		GranularSynth.SetGrainProbability(InGrainProbability);
	});
}

void UGranularSynth::SetGrainEnvelopeType(const EGranularSynthEnvelopeType EnvelopeType)
{
	SynthCommand([this, EnvelopeType]()
	{
		GranularSynth.SetGrainEnvelopeType((Audio::EGrainEnvelopeType)EnvelopeType);
	});		
}

void UGranularSynth::SetPlaybackSpeed(const float InPlayheadRate)
{
	SynthCommand([this, InPlayheadRate]()
	{
		GranularSynth.SetPlaybackSpeed(InPlayheadRate);
	});
}

void UGranularSynth::SetGrainPitch(const float BasePitch, const FVector2D PitchRange)
{
	SynthCommand([this, BasePitch, PitchRange]()
	{
		GranularSynth.SetGrainPitch(BasePitch, PitchRange);
	});
}

void UGranularSynth::SetGrainVolume(const float BaseVolume, FVector2D VolumeRange)
{
	SynthCommand([this, BaseVolume, VolumeRange]()
	{
		GranularSynth.SetGrainVolume(BaseVolume, VolumeRange);
	});
}

void UGranularSynth::SetGrainPan(const float BasePan, FVector2D PanRange)
{
	SynthCommand([this, BasePan, PanRange]()
	{
		GranularSynth.SetGrainPan(BasePan, PanRange);
	});
}

void UGranularSynth::SetGrainDuration(const float BaseDurationMsec, FVector2D DurationRange)
{
	SynthCommand([this, BaseDurationMsec, DurationRange]()
	{
		GranularSynth.SetGrainDuration(BaseDurationMsec, DurationRange);
	});
}

float UGranularSynth::GetSampleDuration() const
{
	return GranularSynth.GetSampleDuration();
}

void UGranularSynth::SetScrubMode(const bool bScrubMode)
{
	SynthCommand([this, bScrubMode]()
	{
		GranularSynth.SetScrubMode(bScrubMode);
	});
}

void UGranularSynth::SetPlayheadTime(const float InPositionSec, const float InLerpTimeSec, EGranularSynthSeekType SeekType)
{
	SynthCommand([this, InPositionSec, InLerpTimeSec, SeekType]()
	{
		GranularSynth.SeekTime(InPositionSec, InLerpTimeSec, (Audio::ESeekType::Type)SeekType);
	});
}


float UGranularSynth::GetCurrentPlayheadTime() const
{
	return GranularSynth.GetCurrentPlayheadTime();
}

bool UGranularSynth::IsLoaded() const
{
	return bIsLoaded;
}
