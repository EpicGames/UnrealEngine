// Copyright Epic Games, Inc. All Rights Reserved.


#include "SynthComponents/SynthComponentMonoWaveTable.h"
#include "DSP/FloatArrayMath.h"
#include "SynthesisModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SynthComponentMonoWaveTable)

UMonoWaveTableSynthPreset::UMonoWaveTableSynthPreset() : PresetName(TEXT("Default"))
, LockKeyframesToGrid(16)
, WaveTableResolution(512)
, bNormalizeWaveTables(true)
, PropertyChangedCallbacks()
, CachedGridSize{ LockKeyframesToGrid }
{
	FRichCurve* DefaultCurvePointer = DefaultCurve.GetRichCurve();

	check(DefaultCurvePointer);

	// triangle
	DefaultCurvePointer->AddKey(0.0f, 0.0f);
	DefaultCurvePointer->AddKey(0.25f, 0.8f);
	DefaultCurvePointer->AddKey(0.5f, 0.0f);
	DefaultCurvePointer->AddKey(0.75f, -0.8f);
	DefaultCurvePointer->AddKey(1.0f, 0.0f);

	bWasLockedToGrid = bLockKeyframesToGridBool;

	CacheAssetData();
}

void UMonoWaveTableSynthPreset::RegisterWTComponentCallback(uint32 ID, TFunction<void(const AssetChangeInfo& ChangeInfo)> Callback)
{
	PropertyChangedCallbacks.Add(ID, MoveTemp(Callback));
}

void UMonoWaveTableSynthPreset::UnRegisterWTSynthComponentCallback(const uint32 ID)
{
	PropertyChangedCallbacks.Remove(ID);
}

#if WITH_EDITOR
void UMonoWaveTableSynthPreset::PostEditChangeChainProperty(FPropertyChangedChainEvent&)
{
	EditChangeInternal();
}
#endif // WITH_EDITOR

void UMonoWaveTableSynthPreset::EditChangeInternal()
{
	AssetChangeInfo ChangeInfo;
	

	if (!WaveTable.Num())
	{
		ChangeInfo.bNeedsFullRebuild = true;
	}

	// resize tangent data if array length increased
	if (CurveBiDirTangents.Num() < WaveTable.Num())
	{
		CurveBiDirTangents.AddZeroed(WaveTable.Num() - CurveBiDirTangents.Num());
	}

	// lock keyframes to grid enabled
	const bool bGridSizeChanged = LockKeyframesToGrid != CachedGridSize;
 	if (bLockKeyframesToGridBool && (!bWasLockedToGrid || bGridSizeChanged))
 	{
 		SampleAllToGrid(LockKeyframesToGrid);
		ChangeInfo.bNeedsFullRebuild = true;
 	}

	// Loop through curve array...
 	// Impose domain/range restrictions to the curves
 	uint32 CurveIndex = 0;
	for (FRuntimeFloatCurve& Curve : WaveTable)
	{
		bool bChangedThisCurve = false;

		if (Curve.EditorCurveData.GetNumKeys() == 0)
		{
			Curve = DefaultCurve;
			bChangedThisCurve = true;
		}

		if (bLockKeyframesToGridBool && Curve.EditorCurveData.GetNumKeys() != CachedGridSize)
		{
			SampleToGrid(LockKeyframesToGrid, CurveIndex);
			bChangedThisCurve = true;
		}

		Curve.EditorCurveData.RemoveRedundantAutoTangentKeys(SMALL_NUMBER, -0.1f, 1.1f);

		// Loop through Keys in this Curve...
		for (auto& Key : Curve.EditorCurveData.Keys)
		{
			// Vert clamp
			float ClampedValue = FMath::Clamp(Key.Value, -1.0f, 1.0f);

			if (!FMath::IsNearlyEqual(ClampedValue, Key.Value)) 
			{
				ERichCurveInterpMode Mode = Key.InterpMode;
				ERichCurveTangentMode Tan = Key.TangentMode;

				Key.Value = ClampedValue;

				Key.InterpMode = Mode;
				Key.TangentMode = Tan;

				bChangedThisCurve = true;
			}
		}
 
		// Horz clamp
		auto FirstKey = Curve.EditorCurveData.GetFirstKeyHandle();
		auto LastKey = Curve.EditorCurveData.GetLastKeyHandle();

		if (Curve.EditorCurveData.GetFirstKey().Time < 0.0)
		{
			Curve.EditorCurveData.SetKeyTime(FirstKey, 0.0);
			bChangedThisCurve = true;
		}
		else if (Curve.EditorCurveData.GetLastKey().Time > 1.0)
		{
			Curve.EditorCurveData.SetKeyTime(LastKey, 1.0);
			bChangedThisCurve = true;
		}

		// Lock nodes horizontally
		if (bLockKeyframesToGridBool)
		{
			const float SamplePeriod = 1.0f / (CachedGridSize - 1);
			auto KeyHandle = Curve.EditorCurveData.GetKeyHandleIterator();

			for (int32 i = 0; i < CachedGridSize; ++i)
			{
				if (!FMath::IsNearlyEqual(Curve.EditorCurveData.GetKeyTime(*KeyHandle), i * SamplePeriod))
				{
					ERichCurveInterpMode Mode = Curve.EditorCurveData.GetKeyInterpMode(*KeyHandle);
					ERichCurveTangentMode Tan = Curve.EditorCurveData.GetKeyTangentMode(*KeyHandle);

					Curve.EditorCurveData.SetKeyTime(*KeyHandle, i * SamplePeriod);

					Curve.EditorCurveData.SetKeyInterpMode(*KeyHandle, Mode);
					Curve.EditorCurveData.SetKeyTangentMode(*KeyHandle, Tan);

					bChangedThisCurve = true;
				}

				++KeyHandle;
			}
		}

		// reset tangent (not a change)
		Curve.EditorCurveData.AutoSetTangents(CurveBiDirTangents[CurveIndex]);

		// Update Change info
		if(bChangedThisCurve)
		{
			ChangeInfo.FlagCurveAsAltered(CurveIndex);
		}

		++CurveIndex;
	}

	// Prepare ChangeInfo struct:

	// Normalization changed
	if (bCachedNormalizationSetting != bNormalizeWaveTables)
	{
		ChangeInfo.bNeedsFullRebuild = true;
	}
	// Table Resolution changed
	else if (CachedTableResolution != WaveTableResolution)
	{
		ChangeInfo.bNeedsFullRebuild = true;
	}
	// Wave Table Data Length Changed
	else if (CachedWaveTable.Num() != WaveTable.Num())
	{
		// TODO: Find out which one was added (Duplicate)
		bool bNewEntryIsLastEntry = true;

		int NumTableEntries = FMath::Min(CachedWaveTable.Num(), WaveTable.Num());
		for (Audio::DefaultWaveTableIndexType i = 0; i < NumTableEntries; ++i)
		{
			if (!IsCachedTableEntryStillValid(i))
			{
				// curve added to (or removed from) middle
				ChangeInfo.bNeedsFullRebuild = true;
				bNewEntryIsLastEntry = false;
				break;
			}
		}

		// New curve added?
		if (CachedWaveTable.Num() == (WaveTable.Num() - 1))
		{
			if (WaveTable.Num())
			{
				if (bNewEntryIsLastEntry)
				{
					// WaveTable QOL: nice to duplicate last curve and make small changes
					DuplicateCurveToEnd();
					ChangeInfo.CurveThatWasAltered = WaveTable.Num() - 1;
				}

				// first table added
				if (WaveTable.Num() == 1 && CachedWaveTable.Num() == 0)
				{
					if(bLockKeyframesToGridBool)
					{
						SampleToGrid(LockKeyframesToGrid, 0);
					}

					ChangeInfo.bNeedsFullRebuild = true;
				}
			}
		}
	}
	else
	{
		// array is the same length as before, find out which index change (loop, compare rich curves)
		int NumTableEntries = WaveTable.Num();
		for (Audio::DefaultWaveTableIndexType i = 0; i < NumTableEntries; ++i)
		{
			if (!IsCachedTableEntryStillValid(i))
			{
				ChangeInfo.FlagCurveAsAltered(i);
				break;
			}
		}
	}

	CacheAssetData();

	// notify all subscribers
	for (const auto& Entry : PropertyChangedCallbacks)
	{
		Entry.Value(ChangeInfo);
	}

	bWasLockedToGrid = bLockKeyframesToGridBool;
	CachedGridSize =LockKeyframesToGrid;

	this->MarkPackageDirty();
}

void UMonoWaveTableSynthPreset::SampleAllToGrid(uint32 InGridsize)
{
	CachedGridSize = InGridsize;

	TArray<float> NewSamples;
	NewSamples.AddZeroed(CachedGridSize);

	const float SamplePeriod = 1.0f / (CachedGridSize - 1);
	
	for (FRuntimeFloatCurve& Curve : WaveTable)
	{
		// sample old curve
		for (int32 i = 0; i < CachedGridSize; ++i)
		{
			NewSamples[i] = Curve.EditorCurveData.Eval(i * SamplePeriod);
		}

		// write samples as new key frames
		Curve.EditorCurveData.Reset();
		for (int32 i = 0; i < CachedGridSize; ++i)
		{
			Curve.EditorCurveData.AddKey(i*SamplePeriod, NewSamples[i]);
		}
	}
}

void UMonoWaveTableSynthPreset::SampleToGrid(uint32 InGridSize, uint32 InTableIndex)
{
	CachedGridSize = InGridSize;

	TArray<float> NewSamples;
	NewSamples.AddZeroed(CachedGridSize);

	const float SamplePeriod = 1.0f / (CachedGridSize - 1);

	FRuntimeFloatCurve& Curve = WaveTable[InTableIndex];

	// sample old curve
	for (int32 i = 0; i < CachedGridSize; ++i)
	{
		NewSamples[i] = Curve.EditorCurveData.Eval(i * SamplePeriod);
	}

	// write samples as new key frames
	Curve.EditorCurveData.Reset();
	for (int32 i = 0; i < CachedGridSize; ++i)
	{
		Curve.EditorCurveData.AddKey(i*SamplePeriod, NewSamples[i]);
	}
}

void UMonoWaveTableSynthPreset::DuplicateCurveToEnd()
{
	if (!WaveTable.Num())
	{
		WaveTable.AddDefaulted();
	}

	if (CurveBiDirTangents.Num() < WaveTable.Num())
	{
		CurveBiDirTangents.AddZeroed(WaveTable.Num() - CurveBiDirTangents.Num());
	}

	// at least 2 elements? (something to copy)
	if (WaveTable.Num() >= 2)
	{
		WaveTable[WaveTable.Num() - 1] = WaveTable[WaveTable.Num() - 2];
		CurveBiDirTangents[WaveTable.Num() - 1] = CurveBiDirTangents[WaveTable.Num() - 2];
	}
	else
	{
		// just in throw default
		WaveTable[WaveTable.Num() - 1] = DefaultCurve;
		CurveBiDirTangents[WaveTable.Num() - 1] = 0.0f;
	}
}

void UMonoWaveTableSynthPreset::CacheAssetData()
{
		bCachedNormalizationSetting = bNormalizeWaveTables;
		CachedWaveTable = WaveTable;
		CachedTableResolution = WaveTableResolution;
}

bool UMonoWaveTableSynthPreset::IsCachedTableEntryStillValid(Audio::DefaultWaveTableIndexType Index)
{
	FRichCurve* OldCurve = CachedWaveTable[Index].GetRichCurve();
	FRichCurve* NewCurve = WaveTable[Index].GetRichCurve();

	if (!OldCurve || !NewCurve)
	{
		return false;
	}

	const bool bIsValid = *OldCurve == *NewCurve;

	return bIsValid;
}

USynthComponentMonoWaveTable::USynthComponentMonoWaveTable(const FObjectInitializer & ObjectInitializer)
	: Super(ObjectInitializer)
	, CachedPreset(nullptr)
{
}

bool USynthComponentMonoWaveTable::Init(int32& InSampleRate)
{
	SampleRate = InSampleRate;
	
	SetSynthPreset(CurrentPreset);

	// Initialize the DSP objects
	SynthCommand([this]
	{
		Synth.Init(SampleRate, GetNumTableEntries(), CachedPreset ? CachedPreset->WaveTableResolution : 256);
	});

	if (CachedPreset)
	{
		// pipe wavetable data to synth
		CachedPreset->CacheAssetData();
		RefreshAllWaveTables();
	}

 	return true;
}
Audio::DefaultWaveTableIndexType USynthComponentMonoWaveTable::OnGenerateAudio(float* OutAudio, int32 NumSamples)
{
	// Perform DSP operations here
	return Synth.OnGenerateAudio(OutAudio, NumSamples);
}

// refresh a particular wavetable (sampled from curve data)
void USynthComponentMonoWaveTable::RefreshWaveTable(int32 Index)
{
	check(Index < CachedPreset->WaveTable.Num())

	// prepare copy of raw curve data for synth
	Audio::DefaultWaveTableIndexType TableRes = static_cast<Audio::DefaultWaveTableIndexType>(CachedPreset->WaveTableResolution);
	FRichCurve* PointerToCurve = CachedPreset->WaveTable[Index].GetRichCurve();
	
	check(PointerToCurve)
	
	FRichCurve RichCurveToProcess = *PointerToCurve;

	RichCurveToProcess.BakeCurve(TableRes, 0.0f, 1.0f);

	// Baked curve -> TArray<float>
	TArray<float> DestinationTable;

	// Vector code will only SIMD aligned portion.
	DestinationTable.SetNumUninitialized(TableRes);

	double SamplePeriod = 1.0 / CachedPreset->WaveTableResolution;
	double CurrTime = 0.0;

	// for (optional) normalization
	float MaxValue = 0.0f;
	float MinValue = 0.0f;

	for (Audio::DefaultWaveTableIndexType i = 0; i < TableRes; ++i)
	{
		DestinationTable[i] = RichCurveToProcess.Eval(CurrTime);
		const float CurrSample = DestinationTable[i];

		MaxValue = CurrSample > MaxValue ? CurrSample : MaxValue;
		MinValue = CurrSample < MinValue ? CurrSample : MinValue;
		CurrTime += SamplePeriod;
	}

	const float DCShift = -0.5f * (FMath::Abs(MaxValue) - FMath::Abs(MinValue));
	MaxValue += DCShift;
	MinValue += DCShift;

	TArrayView<float> DestinationTableView(DestinationTable.GetData(), TableRes);

	// Remove DC offset
	if (!FMath::IsNearlyZero(DCShift))
	{
		Audio::ArrayAddConstantInplace(DestinationTableView, DCShift);
	}

	if (CachedPreset->bNormalizeWaveTables)
	{
		const float MaxAbsValue = FMath::Max(FMath::Abs(MaxValue), FMath::Abs(MinValue));
		const float NormalizationScalar = 1.0f / MaxAbsValue;

		Audio::ArrayMultiplyByConstantInPlace(DestinationTableView, NormalizationScalar);
	}
	else // clip the values since we aren't normalizing
	{
		for (auto& Sample : DestinationTable)
		{
			Sample = FMath::Clamp(Sample, -1.0f, 1.0f);
		}
	}


	// push sampled data to synth
	// (Note: only performs simple memcopy in pump)
	SynthCommand([this, Index, DestinationTable{ MoveTemp(DestinationTable) }]()
	{
		Synth.UpdateWaveTable(Index, static_cast<TArray<float>>(DestinationTable));
	});
}

// refresh all wavetables (from Game Thread data)
void USynthComponentMonoWaveTable::RefreshAllWaveTables()
{
	for (Audio::DefaultWaveTableIndexType i = 0; i < CachedPreset->WaveTable.Num(); ++i)
	{
		RefreshWaveTable(i);
 	}
}

void USynthComponentMonoWaveTable::SetSynthPreset(UMonoWaveTableSynthPreset* SynthPreset)
{
	if (CachedPreset == SynthPreset)
	{
		return;
	}

	if (CachedPreset)
	{
		CachedPreset->UnRegisterWTSynthComponentCallback(GetUniqueID());
	}

	CachedPreset = SynthPreset;

	if (CachedPreset)
	{
		CachedPreset->RegisterWTComponentCallback(GetUniqueID(), { [&](const AssetChangeInfo& ChangeInfo) { ReactToAssetChange(ChangeInfo); } });

		// refresh BP listeners
		this->OnNumTablesChanged.Broadcast();

		for (int i = 0; i < CachedPreset->WaveTable.Num(); ++i)
		{
			this->OnTableAltered.Broadcast(i);
		}
	}

	InitSynth();
}


void USynthComponentMonoWaveTable::SetLowPassFilterFrequency(float InLowPassFilterFrequency)
{
	InLowPassFilterFrequency = FMath::Max(InLowPassFilterFrequency, 5.0f);
	SynthCommand([this, InLowPassFilterFrequency]
	{
		Synth.SetLpfFreq(InLowPassFilterFrequency);
	});
}

void USynthComponentMonoWaveTable::SetLowPassFilterResonance(float InNewQ)
{
	SynthCommand([this, InNewQ]
	{
		Synth.SetLpfRes(InNewQ);
	});
}

// Amp setting
void USynthComponentMonoWaveTable::SetAmpEnvelopeAttackTime(const float InAttackTimeMsec)
{
	SynthCommand([this, InAttackTimeMsec]
	{
		Synth.SetAmpEnvelopeAttackTime(InAttackTimeMsec);
	});
}

void USynthComponentMonoWaveTable::SetAmpEnvelopeDecayTime(const float InDecayTimeMsec)
{
	SynthCommand([this, InDecayTimeMsec]
	{
		Synth.SetAmpEnvelopeDecayTime(InDecayTimeMsec);
	});
}

void USynthComponentMonoWaveTable::SetAmpEnvelopeSustainGain(const float InSustainGain)
{
	SynthCommand([this, InSustainGain]
	{
		Synth.SetAmpEnvelopeSustainGain(InSustainGain);
	});
}

void USynthComponentMonoWaveTable::SetAmpEnvelopeReleaseTime(const float InReleaseTimeMsec)
{
	SynthCommand([this, InReleaseTimeMsec]
	{
		Synth.SetAmpEnvelopeReleaseTime(InReleaseTimeMsec);
	});
}

void USynthComponentMonoWaveTable::SetAmpEnvelopeInvert(const bool bInInvert)
{
	SynthCommand([this, bInInvert]
	{
		Synth.SetAmpEnvelopeInvert(bInInvert);
	});
}

void USynthComponentMonoWaveTable::SetAmpEnvelopeBiasInvert(const bool bInBiasInvert)
{
	SynthCommand([this, bInBiasInvert]
	{
		Synth.SetAmpEnvelopeBiasInvert(bInBiasInvert);
	});
}

void USynthComponentMonoWaveTable::SetAmpEnvelopeDepth(const float InDepth)
{
SynthCommand([this, InDepth]
	{
		Synth.SetAmpEnvelopeDepth(InDepth);
	});
}

void USynthComponentMonoWaveTable::SetAmpEnvelopeBiasDepth(const float InDepth)
{
	SynthCommand([this, InDepth]
	{
		Synth.SetAmpEnvelopeBiasDepth(InDepth);
	});
}

void USynthComponentMonoWaveTable::SetPosLfoFrequency(const float InFrequency)
{
	SynthCommand([this, InFrequency]()
	{
		Synth.SetPosLFOFrequency(InFrequency);
	});
}

void USynthComponentMonoWaveTable::SetPosLfoDepth(const float InLfoDepth)
{
	SynthCommand([this, InLfoDepth]()
	{
			Synth.SetPosLFODepth(InLfoDepth);
	});
}

void USynthComponentMonoWaveTable::SetPosLfoType(const ESynthLFOType InLfoType)
{
	SynthCommand([this, InLfoType]
	{
		Synth.SetPosLFOType((Audio::ELFO::Type)InLfoType);
	});
}

// filter setting
void USynthComponentMonoWaveTable::SetFilterEnvelopeAttackTime(const float InAttackTimeMsec)
{
	SynthCommand([this, InAttackTimeMsec]
	{
		Synth.SetFilterEnvelopeAttackTime(InAttackTimeMsec);
	});
}

void USynthComponentMonoWaveTable::SetFilterEnvelopenDecayTime(const float InDecayTimeMsec)
{
	SynthCommand([this, InDecayTimeMsec]
	{
		Synth.SetFilterEnvelopenDecayTime(InDecayTimeMsec);
	});
}

void USynthComponentMonoWaveTable::SetFilterEnvelopeSustainGain(const float InSustainGain)
{
	SynthCommand([this, InSustainGain]
	{
		Synth.SetFilterEnvelopeSustainGain(InSustainGain);
	});
}

void USynthComponentMonoWaveTable::SetFilterEnvelopeReleaseTime(const float InReleaseTimeMsec)
{
	SynthCommand([this, InReleaseTimeMsec]
	{
		Synth.SetFilterEnvelopeReleaseTime(InReleaseTimeMsec);
	});
}

void USynthComponentMonoWaveTable::SetFilterEnvelopeInvert(const bool bInInvert)
{
	SynthCommand([this, bInInvert]
	{
		Synth.SetFilterEnvelopeInvert(bInInvert);
	});
}

void USynthComponentMonoWaveTable::SetFilterEnvelopeBiasInvert(const bool bInBiasInvert)
{
	SynthCommand([this, bInBiasInvert]
	{
		Synth.SetFilterEnvelopeBiasInvert(bInBiasInvert);
	});
}

void USynthComponentMonoWaveTable::SetFilterEnvelopeDepth(const float InDepth)
{
	SynthCommand([this, InDepth]
	{
		Synth.SetFilterEnvelopeDepth(InDepth);
	});
}

void USynthComponentMonoWaveTable::SetFilterEnvelopeBiasDepth(const float InDepth)
{
	SynthCommand([this, InDepth]
	{
		Synth.SetFilterEnvelopeBiasDepth(InDepth);
	});
}

// position setting
void USynthComponentMonoWaveTable::SetPositionEnvelopeAttackTime(const float InAttackTimeMsec)
{
	SynthCommand([this, InAttackTimeMsec]
	{
		Synth.SetPositionEnvelopeAttackTime(InAttackTimeMsec);
	});
}

void USynthComponentMonoWaveTable::SetPositionEnvelopeDecayTime(const float InDecayTimeMsec)
{
	SynthCommand([this, InDecayTimeMsec]
	{
		Synth.SetPositionEnvelopeDecayTime(InDecayTimeMsec);
	});
}

void USynthComponentMonoWaveTable::SetPositionEnvelopeSustainGain(const float InSustainGain)
{
	SynthCommand([this, InSustainGain]
	{
		Synth.SetPositionEnvelopeSustainGain(InSustainGain);
	});
}

void USynthComponentMonoWaveTable::SetPositionEnvelopeReleaseTime(const float InReleaseTimeMsec)
{
	SynthCommand([this, InReleaseTimeMsec]
	{
		Synth.SetPositionEnvelopeReleaseTime(InReleaseTimeMsec);
	});
}

void USynthComponentMonoWaveTable::SetPositionEnvelopeInvert(const bool bInInvert)
{
	SynthCommand([this, bInInvert]
	{
		Synth.SetPositionEnvelopeInvert(bInInvert);
	});
}

void USynthComponentMonoWaveTable::SetPositionEnvelopeBiasInvert(const bool bInBiasInvert)
{
	SynthCommand([this, bInBiasInvert]
	{
		Synth.SetPositionEnvelopeBiasInvert(bInBiasInvert);
	});
}

void USynthComponentMonoWaveTable::SetPositionEnvelopeDepth(const float InDepth)
{
	SynthCommand([this, InDepth]
	{
		Synth.SetPositionEnvelopeDepth(InDepth);
	});
}

void USynthComponentMonoWaveTable::SetPositionEnvelopeBiasDepth(const float InDepth)
{
	SynthCommand([this, InDepth]
	{
		Synth.SetPositionEnvelopeBiasDepth(InDepth);
	});
}

// performance setting
void USynthComponentMonoWaveTable::NoteOn(const float InMidiNote, const float InVelocity)
{
	const float NewFrequency = Audio::GetFrequencyFromMidi(InMidiNote);
	const float NewGain = Audio::GetGainFromVelocity(InVelocity);

	SynthCommand([this, NewFrequency, NewGain]
	{
		Synth.NoteOn(NewFrequency, NewGain);
	});
}

void USynthComponentMonoWaveTable::NoteOff(const float InMidiNote)
{	
	const float TargetFrequency = Audio::GetFrequencyFromMidi(InMidiNote);

	SynthCommand([this, TargetFrequency]
	{
		Synth.NoteOff(TargetFrequency);
	});
}

void USynthComponentMonoWaveTable::SetFrequency(const float InFrequencyHz)
{
	SynthCommand([this, InFrequencyHz]()
	{
		Synth.SetFrequency(InFrequencyHz);
	});
}

void USynthComponentMonoWaveTable::SetFrequencyPitchBend(const float FrequencyOffsetCents)
{
	SynthCommand([this, FrequencyOffsetCents]
	{
		Synth.SetFrequencyOffset(FrequencyOffsetCents);
	});
}

void USynthComponentMonoWaveTable::SetFrequencyWithMidiNote(const float InMidiNote)
{
	SetFrequency(Audio::GetFrequencyFromMidi(InMidiNote));
}

void USynthComponentMonoWaveTable::SetWaveTablePosition(float InPosition)
{
	SynthCommand([this, InPosition]()
	{
		Synth.SetPosition(InPosition);
	});
}

bool USynthComponentMonoWaveTable::SetCurveValue(int32 TableIndex, int32 KeyframeIndex, const float NewValue)
{
	// Check input
	if (!CachedPreset)
	{
		UE_LOGF(LogSynthesis, Warning, "Trying to edit a Wave Table curve value when there is no Wave Table Preset Asset, returning false");
		return false;
	}
	else if (TableIndex >= CachedPreset->WaveTable.Num())
	{
		UE_LOGF(LogSynthesis, Warning, "Table index: %i is out of bounds! (Number of Curves: %i), returning false"
		, TableIndex, CachedPreset->WaveTable.Num());
		return false;
	}
	else if (KeyframeIndex >= CachedPreset->WaveTable[TableIndex].EditorCurveData.GetNumKeys())
	{
		UE_LOGF(LogSynthesis, Warning, "Frame index: %i is out of bounds (Number of KeyFrames: %i), returning false"
		, KeyframeIndex, CachedPreset->WaveTable[TableIndex].EditorCurveData.GetNumKeys());
		return false;
	}

	// request is valid, execute
	TArray<FRuntimeFloatCurve>& Table = CachedPreset->WaveTable;

	auto KeyIterator = Table[TableIndex].EditorCurveData.GetKeyHandleIterator();

	Table[TableIndex].EditorCurveData.SetKeyValue(*(KeyIterator + KeyframeIndex), NewValue);
	CachedPreset->EditChangeInternal();

	return true;
}

bool USynthComponentMonoWaveTable::SetCurveInterpolationType(CurveInterpolationType InterpolationType, int32 TableIndex)
{
	// Check input
	if (!CachedPreset)
	{
		UE_LOGF(LogSynthesis, Warning, "Trying to edit a Wave Table interpolation type when there is no Wave Table Preset Asset, returning false");
		return false;
	}
	else if (TableIndex >= CachedPreset->WaveTable.Num())
	{
		UE_LOGF(LogSynthesis, Warning, "Table index: %i is out of bounds! (Number of Curves: %i), returning false"
			, TableIndex, CachedPreset->WaveTable.Num());
		return false;
	}

	FRichCurve& TableData = CachedPreset->WaveTable[TableIndex].EditorCurveData;
	auto KeyHandleIter = TableData.GetKeyHandleIterator();

	ERichCurveInterpMode NewInterpMode;
	ERichCurveTangentMode NewTangentMode;

	switch (InterpolationType)
	{
		case CurveInterpolationType::AUTOINTERP :
			NewInterpMode = ERichCurveInterpMode::RCIM_Cubic;
			NewTangentMode = ERichCurveTangentMode::RCTM_Auto;
			break;

		case CurveInterpolationType::LINEAR :
			NewInterpMode = ERichCurveInterpMode::RCIM_Linear;
			NewTangentMode = ERichCurveTangentMode::RCTM_Auto;
			break;
		
		case CurveInterpolationType::CONSTANT :
			NewInterpMode = ERichCurveInterpMode::RCIM_Constant;
			NewTangentMode = ERichCurveTangentMode::RCTM_Auto;
			break;

		default:
			NewInterpMode = ERichCurveInterpMode::RCIM_Linear;
			NewTangentMode = ERichCurveTangentMode::RCTM_Auto;
	}


	for (int i = 0; i < TableData.GetNumKeys(); ++i)
	{
 		TableData.SetKeyInterpMode(*KeyHandleIter, NewInterpMode);
		TableData.SetKeyTangentMode(*KeyHandleIter, NewTangentMode);
 		++KeyHandleIter;
	}

	CachedPreset->EditChangeInternal();

	return true;
}

bool USynthComponentMonoWaveTable::SetCurveTangent(int32 TableIndex, float InNewTangent)
{
	// Check input
	if (!CachedPreset)
	{
		UE_LOGF(LogSynthesis, Warning, "Trying to edit a Wave Table curve tangent value when there is no Wave Table Preset Asset, returning false");
		return false;
	}
	else if (TableIndex >= CachedPreset->WaveTable.Num())
	{
		UE_LOGF(LogSynthesis, Warning, "Table index: %i is out of bounds! (Number of Curves: %i), returning false"
			, TableIndex, CachedPreset->WaveTable.Num());
		return false;
	}

	// set to correct interpolation mode
	FRichCurve& TableData = CachedPreset->WaveTable[TableIndex].EditorCurveData;
	auto KeyHandleIter = TableData.GetKeyHandleIterator();

	for (int i = 0; i < TableData.GetNumKeys(); ++i)
	{
		TableData.SetKeyInterpMode(*KeyHandleIter, ERichCurveInterpMode::RCIM_Cubic);
		TableData.SetKeyTangentMode(*KeyHandleIter, ERichCurveTangentMode::RCTM_Auto);
		++KeyHandleIter;
	}

	if (CachedPreset->CurveBiDirTangents.Num() < CachedPreset->WaveTable.Num())
	{
		CachedPreset->CurveBiDirTangents.AddUninitialized(CachedPreset->WaveTable.Num() - CachedPreset->CurveBiDirTangents.Num());
	}

	CachedPreset->CurveBiDirTangents[TableIndex] = InNewTangent;
	CachedPreset->EditChangeInternal();

	return true;
}

// TODO: Enable this functionality when Curve bug is fixed
/*
bool USynthComponentMonoWaveTable::AddTableToEnd()
{
	if (!CachedPreset)
	{
		return false;
	}

	CachedPreset->WaveTable.AddDefaulted();
	CachedPreset->DuplicateCurveToEnd();
	CachedPreset->EditChangeInternal();

	return true;
}

bool USynthComponentMonoWaveTable::RemoveTableFromEnd()
{
	if (!CachedPreset || (CachedPreset->WaveTable.Num() - 1) <= 0)
	{
		return false;
	}

	CachedPreset->WaveTable.RemoveAt(CachedPreset->WaveTable.Num() - 1 );

	return true;
}
*/

float USynthComponentMonoWaveTable::GetCurveTangent(int32 TableIndex)
{
	// check input
	if (!CachedPreset)
	{
		// no asset
		UE_LOGF(LogSynthesis, Warning, "Trying to get Wave Table Tangent values when there is no Wave Table Curve Asset, returning 0.0");
		return 0.0f;
	}
	else if (CachedPreset->WaveTable.Num() <= TableIndex || CachedPreset->CurveBiDirTangents.Num() <= TableIndex || TableIndex <= -1.0f)
	{
		// Index out of range
		UE_LOGF(LogSynthesis, Warning, "Table index: %i is out of bounds! (Number of Curves: %i), returning 0.0"
			, TableIndex, CachedPreset->WaveTable.Num());
		return 0.0f;
	}

	return CachedPreset->CurveBiDirTangents[TableIndex];
}

void USynthComponentMonoWaveTable::SetSustainPedalState(bool InSustainPedalState)
{
	SynthCommand([this, InSustainPedalState]()
	{
		Synth.SetSustainPedalPressed(InSustainPedalState);
	});
}

TArray<float> USynthComponentMonoWaveTable::GetKeyFrameValuesForTable(float TableIndex) const
{
	TArray<float> Values;
	Values.Reset();

	// check input
	if (!CachedPreset)
	{
		// no asset
		UE_LOGF(LogSynthesis, Warning, "Trying to get Wave Table curve values when there is no Wave Table Curve Asset, returning empty TArray");
		return Values;
	}
	else if (CachedPreset->WaveTable.Num() <= TableIndex || TableIndex < 0.0f)
	{
		// Index out of range
		UE_LOGF(LogSynthesis, Warning, "Table index: %f is out of bounds! (Number of Curves: %i), returning empty TArray"
		, TableIndex, CachedPreset->WaveTable.Num());
		return Values;
	}


	TArray<FRuntimeFloatCurve>& Table = CachedPreset->WaveTable;
	int32 WholeIndex = FMath::FloorToInt(TableIndex);
	int32 NextIndex = WholeIndex + 1;

	if (NextIndex == Table.Num())
	{
		NextIndex = 0;
	}

	auto KeyIteratorA = Table[WholeIndex].EditorCurveData.GetKeyHandleIterator();
	auto KeyIteratorB = Table[NextIndex].EditorCurveData.GetKeyHandleIterator();

	const float Alpha = FMath::Fmod(TableIndex, 1.0f);

	int32 NumKeys = Table[0].EditorCurveData.GetNumKeys();
	Values.Reserve(NumKeys);

	for (int32 i = 0; i < NumKeys; ++i)
	{
		const float ValueA = Table[WholeIndex].EditorCurveData.GetKeyValue(*(KeyIteratorA + i));
		const float ValueB = Table[NextIndex].EditorCurveData.GetKeyValue(*(KeyIteratorB + i));

		Values.Add(FMath::Lerp(ValueA, ValueB, Alpha));
	}

	return Values;
}

void USynthComponentMonoWaveTable::ResetCurve(int32 Index)
{
	if (!CachedPreset)
	{
		return;
	}

	check(Index < CachedPreset->WaveTable.Num());
	CachedPreset->WaveTable[Index] = CachedPreset->DefaultCurve;
}

void USynthComponentMonoWaveTable::InitSynth()
{
	if (!CachedPreset)
	{
		return;
	}

	const int32 NumTableEntries = GetNumTableEntries();

	SynthCommand([this, NumTableEntries]()
	{
		Synth.Init(SampleRate, NumTableEntries, CachedPreset->WaveTableResolution);
	});

	RefreshAllWaveTables();
}

void USynthComponentMonoWaveTable::ReactToAssetChange(const AssetChangeInfo& ChangeInfo)
{
	// this function is called by an Asset object that knows about this USynthComponent.
	// confirm we know about the Asset object.
	check(CachedPreset);

	if (ChangeInfo.bNeedsFullRebuild)
	{
		InitSynth();
	}
	else if (ChangeInfo.CurveThatWasAltered != -1)
	{
		this->OnTableAltered.Broadcast(ChangeInfo.CurveThatWasAltered);
		RefreshWaveTable(ChangeInfo.CurveThatWasAltered);
	}
}

#if WITH_EDITOR
void USynthComponentMonoWaveTable::PostEditChangeChainProperty(FPropertyChangedChainEvent &Event)
{
	SetSynthPreset(CurrentPreset);
}
#endif // WITH_EDITOR

int32 USynthComponentMonoWaveTable::GetNumTableEntries()
{
	if (!CachedPreset)
	{
		return 0;
	}

	return CachedPreset->WaveTable.Num();
}

