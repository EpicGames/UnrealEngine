// Copyright Epic Games, Inc. All Rights Reserved.


#include "SynthComponents/SynthComponentToneGenerator.h"
#include "AudioDevice.h"
#include "Sound/SoundSourceBusSend.h"
#include "Sound/SoundSubmixSend.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SynthComponentToneGenerator)

#define TEST_INT16_ATTENUATION 1

// Disabling this experimental feature for this synth component 
#define SYNTH_COMP_ENABLE_DIST_ATTENUATION 0

FToneGenerator::FToneGenerator(int32 InSampleRate, int32 InNumChannels, int32 InFrequency, float InVolume, const Audio::FAudioBufferDistanceAttenuationSettings& InAttenuationSettings)
	: NumChannels(InNumChannels)
{
	SineOsc.Init(InSampleRate, InFrequency, InVolume);

	DistanceAttenuationSettings = InAttenuationSettings;
}

void FToneGenerator::SetDistance(float InCurrentDistance)
{
	SynthCommand([this, InCurrentDistance]()
	{
		CurrentDistance = InCurrentDistance;
	});
}

void FToneGenerator::SetFrequency(float InFrequency)
{
	SynthCommand([this, InFrequency]()
	{
		SineOsc.SetFrequency(InFrequency);
	});
}

void FToneGenerator::SetVolume(float InVolume)
{
	SynthCommand([this, InVolume]()
	{
		SineOsc.SetScale(InVolume);
	});
}

int32 FToneGenerator::OnGenerateAudio(float* OutAudio, int32 NumSamples)
{
	check(NumChannels != 0);

	int32 NumFrames = NumSamples / NumChannels;
	int32 SampleIndex = 0;

	for (int32 FrameIndex = 0; FrameIndex < NumFrames; ++FrameIndex)
	{
		float Sample = SineOsc.ProcessAudio() * 0.5f;
		for (int32 Channel = 0; Channel < NumChannels; ++Channel)
		{
			OutAudio[SampleIndex++] = Sample;
		}
	}

#if SYNTH_COMP_ENABLE_DIST_ATTENUATION
#if TEST_INT16_ATTENUATION

	// Convert to int16
	TArray<int16> AudioBuffer;
	AudioBuffer.AddUninitialized(NumSamples);
	for ( SampleIndex = 0; SampleIndex < NumSamples; ++SampleIndex)
	{
		AudioBuffer[SampleIndex] = (int16)(OutAudio[SampleIndex] * 32768.0f);
	}

	TArrayView<int16> AudioBufferView = MakeArrayView(AudioBuffer);
	Audio::DistanceAttenuationProcessAudio(AudioBufferView, NumChannels, CurrentDistance, DistanceAttenuationSettings, CurrentAttenuation);

	// Convert back to float
	for (SampleIndex = 0; SampleIndex < NumSamples; ++SampleIndex)
	{
		OutAudio[SampleIndex] = (float)AudioBuffer[SampleIndex] / 32768.0f;
	}
#else
	TArrayView<float> AudioBufferView = MakeArrayView(OutAudio, NumFrames * NumChannels);

	Audio::DistanceAttenuationProcessAudio(AudioBufferView, NumChannels, CurrentDistance, DistanceAttenuationSettings, CurrentAttenuation);
#endif
#endif

	return NumSamples;
}

USynthComponentToneGenerator::USynthComponentToneGenerator(const FObjectInitializer& ObjInitializer)
	: Super(ObjInitializer)
{
	Frequency = 440.0f;
	Volume = 0.5f;
	NumChannels = 1;

	PrimaryComponentTick.bCanEverTick = true;
	PrimaryComponentTick.bStartWithTickEnabled = true;
	PrimaryComponentTick.TickGroup = TG_DuringPhysics;
	bTickInEditor = true;
}

USynthComponentToneGenerator::~USynthComponentToneGenerator()
{
}

void USynthComponentToneGenerator::SetFrequency(float InFrequency)
{
	Frequency = InFrequency;
	if (ToneGenerator.IsValid())
	{
		FToneGenerator* ToneGen = static_cast<FToneGenerator*>(ToneGenerator.Get());
		ToneGen->SetFrequency(InFrequency);
	}
}

void USynthComponentToneGenerator::SetVolume(float InVolume)
{
	Volume = InVolume;
	if (ToneGenerator.IsValid())
	{
		FToneGenerator* ToneGen = static_cast<FToneGenerator*>(ToneGenerator.Get());
		ToneGen->SetVolume(InVolume);
	}
}

void USynthComponentToneGenerator::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	if (ToneGenerator.IsValid())
	{
		FToneGenerator* ToneGen = static_cast<FToneGenerator*>(ToneGenerator.Get());

		FVector Location = GetComponentLocation();
		FAudioDevice* AudioDevice = GetAudioDevice();
		if (AudioDevice)
		{
			float Distance = AudioDevice->GetDistanceToNearestListener(Location);
			ToneGen->SetDistance(Distance);
		}
	}
}


ISoundGeneratorPtr USynthComponentToneGenerator::CreateSoundGenerator(const FSoundGeneratorInitParams& InParams)
{
	DistanceAttenuationSettings.DistanceRange = DistanceRange;
	DistanceAttenuationSettings.AttenuationDbAtMaxRange = AttenuationDbAtMaxRange;

	// Build an audio curve from the DistanceAttenuationCurve

	FRichCurve* RichCurve = DistanceAttenuationCurve.GetRichCurve();

	if (RichCurve->HasAnyData())
	{
		constexpr int32 NumPoints = 10;
		const float DomainDelta = 1.0f / NumPoints;

		float CurrentDomain = 0.0f;

		TArray<FVector2D> Points;
		Points.AddUninitialized(NumPoints + 1);
		for (int32 i = 0; i < NumPoints + 1; ++i)
		{
			float Value = RichCurve->Eval(CurrentDomain);
			CurrentDomain += DomainDelta;
			Points[i] = { CurrentDomain, Value };
		}

		DistanceAttenuationSettings.AttenuationCurve.SetCurvePoints(MoveTemp(Points));
	}

	PrimaryComponentTick.bCanEverTick = true;

	SetComponentTickEnabled(true);
	RegisterComponent();
	Activate();

	return ToneGenerator = ISoundGeneratorPtr(new FToneGenerator(InParams.SampleRate, InParams.NumChannels, Frequency, Volume, DistanceAttenuationSettings));
}
