// Copyright Epic Games, Inc. All Rights Reserved.

#include "SynthComponents/SynthComponentWaveTable.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SynthComponentWaveTable)

USynthSamplePlayer::USynthSamplePlayer(const FObjectInitializer& ObjInitializer)
	: Super(ObjInitializer)
	, SoundWave(nullptr)
	, SampleDurationSec(0.0f)
	, SamplePlaybackProgressSec(0.0f)
	, bIsLoaded(false)
{
	PrimaryComponentTick.bCanEverTick = true;
}

USynthSamplePlayer::~USynthSamplePlayer()
{
}

bool USynthSamplePlayer::Init(int32& SampleRate)
{
	NumChannels = 2;

	SampleBufferReader.Init(SampleRate);

 	if (SoundWave != nullptr)
 	{
		LoadSoundWaveInternal();
 	}

	return true;
}

void USynthSamplePlayer::SetPitch(float InPitch, float InTimeSec)
{
	SynthCommand([this, InPitch, InTimeSec]()
	{
		SampleBufferReader.SetPitch(InPitch, InTimeSec);
	});
}

void USynthSamplePlayer::SeekToTime(float InTimeSecs, ESamplePlayerSeekType InSeekType, bool bWrap)
{
	Audio::ESeekType::Type SeekType;
	switch (InSeekType)
	{
		default:
		case ESamplePlayerSeekType::FromBeginning:
			SeekType = Audio::ESeekType::FromBeginning;
			break;

		case ESamplePlayerSeekType::FromCurrentPosition:
			SeekType = Audio::ESeekType::FromCurrentPosition;
			break;

		case ESamplePlayerSeekType::FromEnd:
			SeekType = Audio::ESeekType::FromEnd;
			break;
	}

	SynthCommand([this, InTimeSecs, SeekType, bWrap]()
	{
		SampleBufferReader.SeekTime(InTimeSecs, SeekType, bWrap);
	});
}

void USynthSamplePlayer::SetScrubMode(bool bScrubMode)
{
	SynthCommand([this, bScrubMode]()
	{
		SampleBufferReader.SetScrubMode(bScrubMode);
	});
}

void USynthSamplePlayer::SetScrubTimeWidth(float InScrubTimeWidthSec)
{
	SynthCommand([this, InScrubTimeWidthSec]()
	{
		SampleBufferReader.SetScrubTimeWidth(InScrubTimeWidthSec);
	});
}

float USynthSamplePlayer::GetSampleDuration() const
{
	return SampleDurationSec;
}

bool USynthSamplePlayer::IsLoaded() const
{
	return bIsLoaded;
}

float USynthSamplePlayer::GetCurrentPlaybackProgressTime() const
{
	return SamplePlaybackProgressSec;
}

float USynthSamplePlayer::GetCurrentPlaybackProgressPercent() const
{
	if (SampleDurationSec > 0.0f)
	{
		return SamplePlaybackProgressSec / SampleDurationSec;
	}
	return 0.0f;
}

void USynthSamplePlayer::LoadSoundWaveInternal()
{
	bIsLoaded = false;

	if (SoundWave)
	{
		TFunction<void(const USoundWave * OutSoundWave, const Audio::FSampleBuffer & OutSampleBuffer)> OnLoaded
			= [this](const USoundWave * OutSoundWave, const Audio::FSampleBuffer & OutSampleBuffer)
		{
			if (OutSoundWave == SoundWave)
			{
				OnSampleLoaded.Broadcast();

				SynthCommand([this, OutSampleBuffer]()
				{
					SampleBuffer = OutSampleBuffer;
					SampleBufferReader.ClearBuffer();
				});

				bIsLoaded = true;
			}
		};

		SoundWaveLoader.LoadSoundWave(SoundWave, MoveTemp(OnLoaded), true);
	}
}

void USynthSamplePlayer::SetSoundWave(USoundWave* InSoundWave)
{
	if (SoundWave != InSoundWave)
	{
		SoundWave = InSoundWave;
		LoadSoundWaveInternal();
	}
}

void USynthSamplePlayer::OnRegister()
{
	Super::OnRegister();

	SetComponentTickEnabled(true);
	RegisterComponent();
}

void USynthSamplePlayer::OnUnregister()
{
	Super::OnUnregister();
}

void USynthSamplePlayer::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);

	SoundWaveLoader.Update();

	OnSamplePlaybackProgress.Broadcast(GetCurrentPlaybackProgressTime(), GetCurrentPlaybackProgressPercent());
}

int32 USynthSamplePlayer::OnGenerateAudio(float* OutAudio, int32 NumSamples)
{
	if (SampleBuffer.GetData() && !SampleBufferReader.HasBuffer())
	{
		const int16* BufferData       = SampleBuffer.GetData();
		const int32 BufferNumSamples  = SampleBuffer.GetNumSamples();
		const int32 BufferNumChannels = SampleBuffer.GetNumChannels();
		const int32 BufferSampleRate  = SampleBuffer.GetSampleRate();

		SampleBufferReader.SetBuffer(BufferData, BufferNumSamples, BufferNumChannels, BufferSampleRate);
		SampleDurationSec = BufferNumSamples / (BufferSampleRate * BufferNumChannels);
	}

	if (SampleBufferReader.HasBuffer())
	{
		const int32 NumFrames = NumSamples / NumChannels;
		SampleBufferReader.Generate(OutAudio, NumFrames, NumChannels, true);
		SamplePlaybackProgressSec = SampleBufferReader.GetPlaybackProgress();
	}
	else
	{
		FMemory::Memzero(OutAudio, NumSamples * sizeof(float));
	}
	return NumSamples;
}

