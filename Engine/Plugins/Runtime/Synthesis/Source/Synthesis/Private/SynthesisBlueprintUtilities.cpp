// Copyright Epic Games, Inc. All Rights Reserved.

#include "SynthesisBlueprintUtilities.h"
#include "DSP/Dsp.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SynthesisBlueprintUtilities)

float USynthesisUtilitiesBlueprintFunctionLibrary::GetLogFrequency(float InLinearValue, float InDomainMin, float InDomainMax, float InRangeMin, float InRangeMax)
{
	return Audio::GetLogFrequencyClamped(InLinearValue, { InDomainMin, InDomainMax }, { InRangeMin, InRangeMax });
}

float USynthesisUtilitiesBlueprintFunctionLibrary::GetLinearFrequency(float InLogFrequencyValue, float InDomainMin, float InDomainMax, float InRangeMin, float InRangeMax)
{
	return Audio::GetLinearFrequencyClamped(InLogFrequencyValue, { InDomainMin, InDomainMax }, { InRangeMin, InRangeMax });
}



