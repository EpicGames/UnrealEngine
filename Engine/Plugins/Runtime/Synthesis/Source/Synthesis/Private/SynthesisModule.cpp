// Copyright Epic Games, Inc. All Rights Reserved.

#include "SynthesisModule.h"

#include "Modules/ModuleManager.h"
#include "UI/SynthSlateStyle.h"

DEFINE_LOG_CATEGORY(LogSynthesis);

IMPLEMENT_MODULE(FSynthesisModule, Synthesis)

void FSynthesisModule::StartupModule()
{
	LLM_SCOPE(ELLMTag::AudioSynthesis);
	FSynthSlateStyleSet::Initialize();

	FModuleManager::Get().LoadModuleChecked(TEXT("SignalProcessing"));
}

void FSynthesisModule::ShutdownModule()
{
	LLM_SCOPE(ELLMTag::AudioSynthesis);
	FSynthSlateStyleSet::Shutdown();
}


