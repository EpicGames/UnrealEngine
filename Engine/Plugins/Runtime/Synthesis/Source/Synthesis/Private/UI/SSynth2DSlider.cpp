// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/SSynth2DSlider.h"
#include "Framework/Application/SlateApplication.h"

void SSynth2DSlider::Construct(const SSynth2DSlider::FArguments& InDeclaration)
{
	check(InDeclaration._Style);

	Style = InDeclaration._Style;

	IndentHandle = InDeclaration._IndentHandle;
	LockedAttribute = InDeclaration._Locked;
	StepSize = InDeclaration._StepSize;
	ValueAttributeX = InDeclaration._ValueX;
	ValueAttributeY = InDeclaration._ValueY;
	bIsFocusable = InDeclaration._IsFocusable;
	OnMouseCaptureBegin = InDeclaration._OnMouseCaptureBegin;
	OnMouseCaptureEnd = InDeclaration._OnMouseCaptureEnd;
	OnControllerCaptureBegin = InDeclaration._OnControllerCaptureBegin;
	OnControllerCaptureEnd = InDeclaration._OnControllerCaptureEnd;
	OnValueChangedX = InDeclaration._OnValueChangedX;
	OnValueChangedY = InDeclaration._OnValueChangedY;

	bControllerInputCaptured = false;
}

int32 SSynth2DSlider::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	const float AllottedWidth = AllottedGeometry.GetLocalSize().X;
	const float AllottedHeight = AllottedGeometry.GetLocalSize().Y;

	FVector2D HandleTopLeftPoint;

	const FVector2D HandleSize = Style->NormalThumbImage.ImageSize;
	const FVector2D HalfHandleSize = 0.5f * HandleSize;
	const float IndentationX = IndentHandle.Get() ? HalfHandleSize.X : 0.0f;
	const float IndentationY = IndentHandle.Get() ? HalfHandleSize.Y : 0.0f;

	const float ValueX = ValueAttributeX.Get();
	const float ValueY = ValueAttributeY.Get();

	const float SliderLengthX = AllottedWidth - IndentationX;
	const float SliderPercentX = ValueX;
	const float SliderHandleOffsetX = SliderPercentX * SliderLengthX;

	const float SliderLengthY = AllottedHeight - IndentationY;
	const float SliderPercentY = ValueY;
	const float SliderHandleOffsetY = SliderPercentY * SliderLengthY;

	FGeometry SliderGeometry = AllottedGeometry;
	const bool bEnabled = ShouldBeEnabled(bParentEnabled);
	const ESlateDrawEffect DrawEffects = bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

	const FLinearColor FinalColorAndOpacity(InWidgetStyle.GetColorAndOpacityTint() * Style->BackgroundImage.GetTint(InWidgetStyle));
	FSlateDrawElement::MakeBox(OutDrawElements, LayerId++, AllottedGeometry.ToPaintGeometry(), &Style->BackgroundImage, DrawEffects, FinalColorAndOpacity);

	HandleTopLeftPoint = FVector2D(SliderHandleOffsetX - (HandleSize.X * SliderPercentX) + 0.5f * IndentationX,
		SliderHandleOffsetY - (HandleSize.Y * SliderPercentY) + 0.5f * IndentationY);

	// draw slider thumb
	FSlateDrawElement::MakeBox(
		OutDrawElements,
		LayerId++,
		SliderGeometry.ToPaintGeometry(Style->NormalThumbImage.ImageSize, FSlateLayoutTransform(HandleTopLeftPoint)),
		LockedAttribute.Get() ? &Style->DisabledThumbImage : &Style->NormalThumbImage,
		DrawEffects,
		SliderHandleColor.Get().GetColor(InWidgetStyle) * InWidgetStyle.GetColorAndOpacityTint()
		);

	return LayerId;
}

FVector2D SSynth2DSlider::ComputeDesiredSize(float) const
{
	static const FVector2D SSynth2DSliderDesiredSize(16.0f, 16.0f);

	if (Style == nullptr)
	{
		return SSynth2DSliderDesiredSize;
	}

	const float Thickness = FMath::Max(Style->BarThickness, Style->NormalThumbImage.ImageSize.Y);

	if (Orientation == Orient_Vertical)
	{
		return FVector2D(Thickness, SSynth2DSliderDesiredSize.Y);
	}

	return FVector2D(SSynth2DSliderDesiredSize.X, Thickness);
}


bool SSynth2DSlider::IsLocked() const
{
	return LockedAttribute.Get();
}

bool SSynth2DSlider::IsInteractable() const
{
	return IsEnabled() && !IsLocked() && SupportsKeyboardFocus();
}

bool SSynth2DSlider::SupportsKeyboardFocus() const
{
	return bIsFocusable;
}

void SSynth2DSlider::ResetControllerState()
{
	if (bControllerInputCaptured)
	{
		OnControllerCaptureEnd.ExecuteIfBound();
		bControllerInputCaptured = false;
	}
}

FReply SSynth2DSlider::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	FReply Reply = FReply::Unhandled();
	const EUINavigation NavDirection = FSlateApplication::Get().GetNavigationDirectionFromKey(InKeyEvent);

	if (IsInteractable())
	{
		// The controller's bottom face button must be pressed once to begin manipulating the slider's value.
		// Navigation away from the widget is prevented until the button has been pressed again or focus is lost.
		// The value can be manipulated by using the game pad's directional arrows ( relative to slider orientation ).
		if (FSlateApplication::Get().GetNavigationActionFromKey(InKeyEvent) == EUINavigationAction::Accept)
		{
			if (bControllerInputCaptured == false)
			{
				// Begin capturing controller input and allow user to modify the slider's value.
				bControllerInputCaptured = true;
				OnControllerCaptureBegin.ExecuteIfBound();
				Reply = FReply::Handled();
			}
			else
			{
				ResetControllerState();
				Reply = FReply::Handled();
			}
		}

		if (bControllerInputCaptured)
		{
			float NewValueX = ValueAttributeX.Get();
			float NewValueY = ValueAttributeY.Get();

			if (Orientation == EOrientation::Orient_Horizontal)
			{
				if (NavDirection == EUINavigation::Left)
				{
					NewValueX -= StepSize.Get();
				}
				else if (NavDirection == EUINavigation::Right)
				{
					NewValueX += StepSize.Get();
				}
			}
			else
			{
				if (NavDirection == EUINavigation::Down)
				{
					NewValueY -= StepSize.Get();
				}
				else if (NavDirection == EUINavigation::Up)
				{
					NewValueY += StepSize.Get();
				}
			}

			CommitValue(FMath::Clamp(NewValueX, 0.0f, 1.0f), FMath::Clamp(NewValueY, 0.0f, 1.0f));
			Reply = FReply::Handled();
		}
		else
		{
			Reply = SLeafWidget::OnKeyDown(MyGeometry, InKeyEvent);
		}
	}
	else
	{
		Reply = SLeafWidget::OnKeyDown(MyGeometry, InKeyEvent);
	}

	return Reply;
}

FReply SSynth2DSlider::OnKeyUp(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	FReply Reply = FReply::Unhandled();
	if (bControllerInputCaptured)
	{
		Reply = FReply::Handled();
	}
	return Reply;
}

void SSynth2DSlider::OnFocusLost(const FFocusEvent& InFocusEvent)
{
	if (bControllerInputCaptured)
	{
		// Commit and reset state
		CommitValue(ValueAttributeX.Get(), ValueAttributeY.Get());
		ResetControllerState();
	}
}

FReply SSynth2DSlider::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if ((MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton) && !IsLocked())
	{
		OnMouseCaptureBegin.ExecuteIfBound();
		FVector2D Value2D = PositionToValue(MyGeometry, MouseEvent.GetLastScreenSpacePosition());
		CommitValue(Value2D.X, Value2D.Y);

		// Release capture for controller/keyboard when switching to mouse.
		ResetControllerState();

		return FReply::Handled().CaptureMouse(SharedThis(this));
	}

	return FReply::Unhandled();
}

FReply SSynth2DSlider::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if ((MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton) && HasMouseCapture())
	{
		OnMouseCaptureEnd.ExecuteIfBound();

		// Release capture for controller/keyboard when switching to mouse.
		ResetControllerState();

		return FReply::Handled().ReleaseMouseCapture();
	}

	return FReply::Unhandled();
}

FReply SSynth2DSlider::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (this->HasMouseCapture() && !IsLocked())
	{
		FVector2D Vector2D = PositionToValue(MyGeometry, MouseEvent.GetLastScreenSpacePosition());

		CommitValue(Vector2D.X, Vector2D.Y);

		// Release capture for controller/keyboard when switching to mouse
		ResetControllerState();

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SSynth2DSlider::CommitValue(float NewValueX, float NewValueY)
{
	if (!ValueAttributeX.IsBound())
	{
		ValueAttributeX.Set(NewValueX);
	}

	if (!ValueAttributeY.IsBound())
	{
		ValueAttributeY.Set(NewValueY);
	}

	OnValueChangedX.ExecuteIfBound(NewValueX);
	OnValueChangedY.ExecuteIfBound(NewValueY);
}

FVector2D SSynth2DSlider::PositionToValue(const FGeometry& MyGeometry, const FVector2D& AbsolutePosition)
{
	const FVector2D LocalPosition = MyGeometry.AbsoluteToLocal(AbsolutePosition);
	const float IndentationX = Style->NormalThumbImage.ImageSize.X;
	const float IndentationY = Style->NormalThumbImage.ImageSize.Y;

	float RelativeValueX = (LocalPosition.X - 0.5f * IndentationX) / (MyGeometry.Size.X - IndentationX);;
	float RelativeValueY = (LocalPosition.Y - 0.5f * IndentationY) / (MyGeometry.Size.Y - IndentationY);;

	RelativeValueX = FMath::Clamp(RelativeValueX, 0.0f, 1.0f);
	RelativeValueY = FMath::Clamp(RelativeValueY, 0.0f, 1.0f);

	return FVector2D(RelativeValueX, RelativeValueY);
}

float SSynth2DSlider::GetValueX() const
{
	return ValueAttributeX.Get();
}

float SSynth2DSlider::GetValueY() const
{
	return ValueAttributeY.Get();
}

void SSynth2DSlider::SetValueX(const TAttribute<float>& InValueAttribute)
{
	ValueAttributeX = InValueAttribute;
}

void SSynth2DSlider::SetValueY(const TAttribute<float>& InValueAttribute)
{
	ValueAttributeY = InValueAttribute;
}

void SSynth2DSlider::SetIndentHandle(const TAttribute<bool>& InIndentHandle)
{
	IndentHandle = InIndentHandle;
}

void SSynth2DSlider::SetLocked(const TAttribute<bool>& InLocked)
{
	LockedAttribute = InLocked;
}

void SSynth2DSlider::SetOrientation(EOrientation InOrientation)
{
	Orientation = InOrientation;
}

void SSynth2DSlider::SetSliderBarColor(FSlateColor InSliderBarColor)
{
	SliderBarColor = InSliderBarColor;
}

void SSynth2DSlider::SetSliderHandleColor(FSlateColor InSliderHandleColor)
{
	SliderHandleColor = InSliderHandleColor;
}

void SSynth2DSlider::SetStepSize(const TAttribute<float>& InStepSize)
{
	StepSize = InStepSize;
}
