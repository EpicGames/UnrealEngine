// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/SSynthKnob.h"
#include "Framework/Application/SlateApplication.h"

#define LOCTEXT_NAMESPACE "SynthKnob"

void SSynthKnob::Construct(const SSynthKnob::FArguments& InDeclaration)
{
	check(InDeclaration._Style);

	Style = InDeclaration._Style;

	LockedAttribute = InDeclaration._Locked;
	ValueAttribute = InDeclaration._Value;
	bIsFocusable = InDeclaration._IsFocusable;
	MouseSpeed = InDeclaration._MouseSpeed;
	MouseFineTuneSpeed = InDeclaration._MouseFineTuneSpeed;
	StepSize = InDeclaration._StepSize;
	OnMouseCaptureBegin = InDeclaration._OnMouseCaptureBegin;
	OnMouseCaptureEnd = InDeclaration._OnMouseCaptureEnd;
	OnControllerCaptureBegin = InDeclaration._OnControllerCaptureBegin;
	OnControllerCaptureEnd = InDeclaration._OnControllerCaptureEnd;
	OnValueChanged = InDeclaration._OnValueChanged;

	ParameterName = InDeclaration._ParameterName;
	ParameterUnits = InDeclaration._ParameterUnits;
	ParameterRange = InDeclaration._ParameterRange;
	ShowTooltip = InDeclaration._ShowParamTooltip;

	MouseDownValue = 0.0f;
	PixelDelta = 50;
	bIsFineTune = false;
	bIsMouseDown = false;
	FineTuneKey = EKeys::LeftShift;

	bControllerInputCaptured = false;

	// independently create a synth tooltip slate object (not a child)
}

int32 SSynthKnob::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	// Figure out which image to use based on the current value
	const float KnobPercent = ValueAttribute.Get();

	const bool bIsEnabled = ShouldBeEnabled(bParentEnabled);
	const ESlateDrawEffect DrawEffects = bIsEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

	const FSlateBrush* BaseImageBrush = Style->GetBaseBrush();
	const FSlateBrush* OverlayImageBrush = Style->GetOverlayBrush();

	if (BaseImageBrush != nullptr)
	{
		const FLinearColor FinalColorAndOpacity(InWidgetStyle.GetColorAndOpacityTint() * BaseImageBrush->GetTint(InWidgetStyle));
		FSlateDrawElement::MakeBox(OutDrawElements, LayerId++, AllottedGeometry.ToPaintGeometry(), BaseImageBrush, DrawEffects, FinalColorAndOpacity);
	}

	if (OverlayImageBrush != nullptr)
	{
		const float MinValueAngle = Style->MinValueAngle;
		const float MaxValueAngle = Style->MaxValueAngle;
		const float CurrentValue = GetValue();
		const FVector2D ImageCenter = AllottedGeometry.GetLocalSize() * 0.5f;

		float NormalizedRotationAngle = CurrentValue * (MaxValueAngle - MinValueAngle) + MinValueAngle;
		float RotationAngle = 2.0f * PI * NormalizedRotationAngle;
		
		const FLinearColor FinalColorAndOpacity(InWidgetStyle.GetColorAndOpacityTint() * OverlayImageBrush->GetTint(InWidgetStyle));
		FSlateDrawElement::MakeRotatedBox(OutDrawElements, LayerId++, AllottedGeometry.ToPaintGeometry(), OverlayImageBrush, DrawEffects, RotationAngle, ImageCenter, FSlateDrawElement::RelativeToElement, FinalColorAndOpacity);
	}

	return LayerId;
}

FVector2D SSynthKnob::ComputeDesiredSize(float LayoutScaleMultiplier) const
{
	const FSlateBrush* ImageBrush = Style->GetBaseBrush();
	if (ImageBrush != nullptr)
	{
		return ImageBrush->ImageSize;
	}
	return FVector2D::ZeroVector;
}

bool SSynthKnob::IsLocked() const
{
	return LockedAttribute.Get();
}

bool SSynthKnob::IsInteractable() const
{
	return IsEnabled() && !IsLocked() && SupportsKeyboardFocus();
}

bool SSynthKnob::SupportsKeyboardFocus() const
{
	return bIsFocusable;
}

void SSynthKnob::ResetControllerState()
{
	if (bControllerInputCaptured)
	{
		OnControllerCaptureEnd.ExecuteIfBound();
		bControllerInputCaptured = false;
	}
}

FReply SSynthKnob::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	FReply Reply = FReply::Unhandled();
	const EUINavigation NavDirection = FSlateApplication::Get().GetNavigationDirectionFromKey(InKeyEvent);

	if (InKeyEvent.GetKey() == FineTuneKey)
	{
		bIsFineTune = true;
	}

	if (IsInteractable())
	{
		// The controller's bottom face button must be pressed once to begin manipulating the slider's value.
		// Navigation away from the widget is prevented until the button has been pressed again or focus is lost.
		// The value can be manipulated by using the game pad's directional arrows ( relative to slider orientation ).
		if (FSlateApplication::Get().GetNavigationActionFromKey(InKeyEvent) == EUINavigationAction::Accept)
		{
			if (bControllerInputCaptured == false)
			{
				// Begin capturing controller input and allow user to modify the slider's value.
				bControllerInputCaptured = true;
				OnControllerCaptureBegin.ExecuteIfBound();
				Reply = FReply::Handled();
			}
			else
			{
				ResetControllerState();
				Reply = FReply::Handled();
			}
		}

		if (bControllerInputCaptured)
		{
			float NewValue = ValueAttribute.Get();
			if (NavDirection == EUINavigation::Down)
			{
				NewValue -= StepSize.Get();
			}
			else if (NavDirection == EUINavigation::Up)
			{
				NewValue += StepSize.Get();
			}

			CommitValue(FMath::Clamp(NewValue, 0.0f, 1.0f));
			Reply = FReply::Handled();
		}
		else
		{
			Reply = SWidget::OnKeyDown(MyGeometry, InKeyEvent);
		}
	}
	else
	{
		Reply = SWidget::OnKeyDown(MyGeometry, InKeyEvent);
	}

	return Reply;
}

FReply SSynthKnob::OnKeyUp(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	FReply Reply = FReply::Unhandled();

	bIsFineTune = false;

	if (bControllerInputCaptured)
	{
		Reply = FReply::Handled();
	}
	return Reply;
}

void SSynthKnob::OnFocusLost(const FFocusEvent& InFocusEvent)
{
	if (bControllerInputCaptured)
	{
		// Commit and reset state
		CommitValue(ValueAttribute.Get());
		ResetControllerState();
	}
}

FReply SSynthKnob::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if ((MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton) && !IsLocked())
	{
		OnMouseCaptureBegin.ExecuteIfBound();

		bIsMouseDown = true;
		MouseDownPosition = MouseEvent.GetScreenSpacePosition();
		MouseDownValue = ValueAttribute.Get();

		// Release capture for controller/keyboard when switching to mouse.
		ResetControllerState();

		FReply Reply = FReply::Handled().CaptureMouse(SharedThis(this));
		bool bHasMouseCapture = this->HasMouseCapture();

		return Reply;
	}

	return FReply::Unhandled();
}

FReply SSynthKnob::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if ((MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton) && this->HasMouseCapture())
	{
		OnMouseCaptureEnd.ExecuteIfBound();

		bIsMouseDown = false;

		// Release capture for controller/keyboard when switching to mouse.
		ResetControllerState();

		return FReply::Handled().ReleaseMouseCapture();
	}

	return FReply::Unhandled();
}

FReply SSynthKnob::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (this->HasMouseCapture() && !IsLocked())
	{
		int32 CurrentYValue = MouseEvent.GetLastScreenSpacePosition().Y;
		
		//float MouseSpeedClamped = bIsFineTune ? MouseFineTuneSpeed.Get() : MouseSpeed.Get();
		//MouseSpeedClamped = FMath::Max(MouseSpeedClamped, 0.01f);

		// TODO: this is hardcoded for demo 
		const float MouseSpeedClamped = bIsFineTune ? 0.05f : 0.2f;

		float ValueDelta = (float) (MouseDownPosition.Y - CurrentYValue) / PixelDelta * MouseSpeedClamped;

		float NewValue = FMath::Clamp(MouseDownValue + ValueDelta, 0.0f, 1.0f);
		CommitValue(NewValue);

		// Release capture for controller/keyboard when switching to mouse
		ResetControllerState();

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SSynthKnob::CommitValue(float NewValue)
{
	if (!ValueAttribute.IsBound())
	{
		ValueAttribute.Set(NewValue);
	}

	OnValueChanged.ExecuteIfBound(NewValue);
}

float SSynthKnob::GetValue() const
{
	return ValueAttribute.Get();
}

void SSynthKnob::SetValue(const TAttribute<float>& InValueAttribute)
{
	ValueAttribute = InValueAttribute;
}

void SSynthKnob::SetLocked(const TAttribute<bool>& InLocked)
{
	LockedAttribute = InLocked;
}

void SSynthKnob::SetStepSize(const float InStepSize)
{
	StepSize = InStepSize;
}

void SSynthKnob::SetMouseSpeed(const float InMouseSpeed)
{
	MouseSpeed = InMouseSpeed;
}

void SSynthKnob::SetMouseFineTuneSpeed(const float InMouseFineTuneSpeed)
{
	MouseFineTuneSpeed = InMouseFineTuneSpeed;
}

FVector2D SSynthKnob::GetMouseDownPosition()
{
	return MouseDownPosition;
}


#undef LOCTEXT_NAMESPACE
