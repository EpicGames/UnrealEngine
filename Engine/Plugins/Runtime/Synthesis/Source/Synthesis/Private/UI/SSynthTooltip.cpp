// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/SSynthTooltip.h"
#include "Widgets/SToolTip.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"

SSynthTooltip::~SSynthTooltip()
{
	if (WindowContainer.IsValid())
	{
		WindowContainer->RequestDestroyWindow();
	}
	WindowContainer.Reset();
}

void SSynthTooltip::Construct(const FArguments& InArgs)
{
	// Show window at the cursor
	WindowContainer = SWindow::MakeCursorDecorator();
	FSlateApplication::Get().AddWindow(WindowContainer.ToSharedRef(), false);

	TooltipText = SNew(STextBlock)
		.Text(FText::GetEmpty());

	WindowContainer->SetContent(
		// Im using Stooltip only because it looks nicer than empty window
		SNew(SToolTip)
		.Content()
		[
			TooltipText.ToSharedRef()
		]);


	Children.AddSlots(MoveTemp(const_cast<TArray<SOverlay::FOverlaySlot::FSlotArguments>&>(InArgs._Slots)));

	bIsVisible = false;
}

void SSynthTooltip::SetWindowContainerVisibility(bool bShowVisibility)
{
	if (WindowContainer.IsValid())
	{
		if (bShowVisibility && !bIsVisible)
		{
			bIsVisible = true;
			WindowContainer->MoveWindowTo(WindowPosition);
			WindowContainer->ShowWindow();
		}
		else if (!bShowVisibility && bIsVisible)
		{
			bIsVisible = false;
			WindowContainer->HideWindow();
		}
	}
}

void SSynthTooltip::SetOverlayWindowPosition(FVector2D Position)
{
	WindowPosition = Position;
	WindowContainer->MoveWindowTo(WindowPosition);
}

void SSynthTooltip::SetOverlayText(const FText& InText)
{
	if (TooltipText.IsValid())
	{
		TooltipText->SetText(InText);
	}
}
