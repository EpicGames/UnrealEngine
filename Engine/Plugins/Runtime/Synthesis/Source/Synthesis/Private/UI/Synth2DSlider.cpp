// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Synth2DSlider.h"
#include "UI/SSynth2DSlider.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(Synth2DSlider)

#define LOCTEXT_NAMESPACE "Synthesis"


USynth2DSlider::USynth2DSlider(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	FSynth2DSliderStyle::Initialize();

	SliderHandleColor = FLinearColor::White;
	StepSize = 0.01f;
	SSynth2DSlider::FArguments Defaults;
	WidgetStyle = *Defaults._Style;
	IsFocusable = true;
}

TSharedRef<SWidget> USynth2DSlider::RebuildWidget()
{
	MySlider = SNew(SSynth2DSlider)
		.Style(&WidgetStyle)
		.IsFocusable(IsFocusable)
		.OnMouseCaptureBegin(BIND_UOBJECT_DELEGATE(FSimpleDelegate, HandleOnMouseCaptureBegin))
		.OnMouseCaptureEnd(BIND_UOBJECT_DELEGATE(FSimpleDelegate, HandleOnMouseCaptureEnd))
		.OnControllerCaptureBegin(BIND_UOBJECT_DELEGATE(FSimpleDelegate, HandleOnControllerCaptureBegin))
		.OnControllerCaptureEnd(BIND_UOBJECT_DELEGATE(FSimpleDelegate, HandleOnControllerCaptureEnd))
		.OnValueChangedX(BIND_UOBJECT_DELEGATE(FOnFloatValueChanged, HandleOnValueChangedX))
		.OnValueChangedY(BIND_UOBJECT_DELEGATE(FOnFloatValueChanged, HandleOnValueChangedY));

	return MySlider.ToSharedRef();
}

void USynth2DSlider::SynchronizeProperties()
{
	Super::SynchronizeProperties();

	TAttribute<float> ValueXBinding = PROPERTY_BINDING(float, ValueX);
	TAttribute<float> ValueYBinding = PROPERTY_BINDING(float, ValueY);

	MySlider->SetSliderHandleColor(SliderHandleColor);
	MySlider->SetValueX(ValueXBinding);
	MySlider->SetValueY(ValueYBinding);
	MySlider->SetLocked(Locked);
	MySlider->SetIndentHandle(IndentHandle);
	MySlider->SetStepSize(StepSize);
}

void USynth2DSlider::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	MySlider.Reset();
}

void USynth2DSlider::HandleOnValueChangedX(float InValue)
{
	OnValueChangedX.Broadcast(InValue);
}

void USynth2DSlider::HandleOnValueChangedY(float InValue)
{
	OnValueChangedY.Broadcast(InValue);
}

void USynth2DSlider::HandleOnMouseCaptureBegin()
{
	OnMouseCaptureBegin.Broadcast();
}

void USynth2DSlider::HandleOnMouseCaptureEnd()
{
	OnMouseCaptureEnd.Broadcast();
}

void USynth2DSlider::HandleOnControllerCaptureBegin()
{
	OnControllerCaptureBegin.Broadcast();
}

void USynth2DSlider::HandleOnControllerCaptureEnd()
{
	OnControllerCaptureEnd.Broadcast();
}

FVector2D USynth2DSlider::GetValue() const
{
	if (MySlider.IsValid())
	{
		FVector2D OutValue;
		OutValue.X = MySlider->GetValueX();
		OutValue.Y = MySlider->GetValueY();
		return OutValue;
	}

	return FVector2D::ZeroVector;
}

void USynth2DSlider::SetValue(FVector2D InValue)
{
	ValueX = InValue.X;
	ValueY = InValue.Y;
	if (MySlider.IsValid())
	{
		MySlider->SetValueX(InValue.X);
		MySlider->SetValueY(InValue.Y);
	}
}

void USynth2DSlider::SetIndentHandle(bool InIndentHandle)
{
	IndentHandle = InIndentHandle;
	if (MySlider.IsValid())
	{
		MySlider->SetIndentHandle(InIndentHandle);
	}
}

void USynth2DSlider::SetLocked(bool InLocked)
{
	Locked = InLocked;
	if (MySlider.IsValid())
	{
		MySlider->SetLocked(InLocked);
	}
}

void USynth2DSlider::SetStepSize(float InValue)
{
	StepSize = InValue;
	if (MySlider.IsValid())
	{
		MySlider->SetStepSize(InValue);
	}
}

void USynth2DSlider::SetSliderHandleColor(FLinearColor InValue)
{
	SliderHandleColor = InValue;
	if (MySlider.IsValid())
	{
		MySlider->SetSliderHandleColor(InValue);
	}
}

#if WITH_EDITOR

const FText USynth2DSlider::GetPaletteCategory()
{
	return LOCTEXT("Synth", "Synth");
}

#endif

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

