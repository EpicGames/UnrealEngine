// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Synth2DSliderStyle.h"
#include "Styling/SlateStyle.h"
#include "UI/SynthSlateStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(Synth2DSliderStyle)

FSynth2DSliderStyle::FSynth2DSliderStyle()
	: BarThickness(2.0f)
{
}

FSynth2DSliderStyle::~FSynth2DSliderStyle()
{
}

void FSynth2DSliderStyle::Initialize()
{
	//first make sure the style set is setup.  Need to make sure because some things happen before StartupModule
	FSynthSlateStyleSet::Initialize();

	FSynthSlateStyleSet::Get()->Set(FSynth2DSliderStyle::TypeName, FSynth2DSliderStyle::GetDefault());
}


void FSynth2DSliderStyle::GetResources(TArray< const FSlateBrush* >& OutBrushes) const
{
}

const FSynth2DSliderStyle& FSynth2DSliderStyle::GetDefault()
{
	static FSynth2DSliderStyle Default;
	return Default;
}

const FName FSynth2DSliderStyle::TypeName(TEXT("Synth2DSliderStyle"));

