// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/SynthKnob.h"
#include "UI/SSynthKnob.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SynthKnob)

#define LOCTEXT_NAMESPACE "UMG"

USynthKnob::USynthKnob(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	FSynthKnobStyle::Initialize();

	StepSize = 0.01f;
	MouseSpeed = 1.0f;
	MouseFineTuneSpeed = 0.2f;
	SSynthKnob::FArguments Defaults;
	WidgetStyle = *Defaults._Style;
	IsFocusable = true;
}

TSharedRef<SWidget> USynthKnob::RebuildWidget()
{
	MySynthKnob = SNew(SSynthKnob)
		.Style(&WidgetStyle)
		.IsFocusable(IsFocusable)
		.OnMouseCaptureBegin(BIND_UOBJECT_DELEGATE(FSimpleDelegate, HandleOnMouseCaptureBegin))
		.OnMouseCaptureEnd(BIND_UOBJECT_DELEGATE(FSimpleDelegate, HandleOnMouseCaptureEnd))
		.OnControllerCaptureBegin(BIND_UOBJECT_DELEGATE(FSimpleDelegate, HandleOnControllerCaptureBegin))
		.OnControllerCaptureEnd(BIND_UOBJECT_DELEGATE(FSimpleDelegate, HandleOnControllerCaptureEnd))
		.OnValueChanged(BIND_UOBJECT_DELEGATE(FOnFloatValueChanged, HandleOnValueChanged));

	return MySynthKnob.ToSharedRef();
}

void USynthKnob::SynchronizeProperties()
{
	Super::SynchronizeProperties();

	TAttribute<float> ValueBinding = PROPERTY_BINDING(float, Value);

	MySynthKnob->SetValue(ValueBinding);
	MySynthKnob->SetLocked(Locked);
	MySynthKnob->SetStepSize(StepSize);
	MySynthKnob->SetMouseSpeed(MouseSpeed);
	MySynthKnob->SetMouseFineTuneSpeed(MouseFineTuneSpeed);
}

void USynthKnob::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	MySynthKnob.Reset();
}

void USynthKnob::HandleOnValueChanged(float InValue)
{
	OnValueChanged.Broadcast(InValue);
}

void USynthKnob::HandleOnMouseCaptureBegin()
{
	OnMouseCaptureBegin.Broadcast();

	UPanelWidget* PanelWidget = GetParent();
}

void USynthKnob::HandleOnMouseCaptureEnd()
{
	OnMouseCaptureEnd.Broadcast();
}

void USynthKnob::HandleOnControllerCaptureBegin()
{
	OnControllerCaptureBegin.Broadcast();
}

void USynthKnob::HandleOnControllerCaptureEnd()
{
	OnControllerCaptureEnd.Broadcast();
}

float USynthKnob::GetValue() const
{
	if (MySynthKnob.IsValid())
	{
		return MySynthKnob->GetValue();
	}

	return Value;
}

void USynthKnob::SetValue(float InValue)
{
	Value = InValue;
	if (MySynthKnob.IsValid())
	{
		MySynthKnob->SetValue(InValue);
	}
}

void USynthKnob::SetLocked(bool InLocked)
{
	Locked = InLocked;
	if (MySynthKnob.IsValid())
	{
		MySynthKnob->SetLocked(InLocked);
	}
}

void USynthKnob::SetStepSize(float InValue)
{
	StepSize = InValue;
	if (MySynthKnob.IsValid())
	{
		MySynthKnob->SetStepSize(InValue);
	}
}

#if WITH_EDITOR

const FText USynthKnob::GetPaletteCategory()
{
	return  LOCTEXT("Synth", "Synth");
}

#endif

#undef LOCTEXT_NAMESPACE


