// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/SynthKnobStyle.h"
#include "Interfaces/IPluginManager.h"
#include "Styling/SlateStyle.h"
#include "UI/SynthSlateStyle.h"
#include "Brushes/SlateDynamicImageBrush.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SynthKnobStyle)

struct FSynthKnobResources
{
	FSynthKnobResources()
		: bImagesLoaded(false)
	{
	}

	void LoadImages()
	{
		if (!bImagesLoaded)
		{
			bImagesLoaded = true;

			FString SynthesisPluginBaseDir = IPluginManager::Get().FindPlugin(TEXT("Synthesis"))->GetBaseDir();
			SynthesisPluginBaseDir += "/Content/UI/";

			FString BrushPath = SynthesisPluginBaseDir + "SynthKnobLarge.png";
			DefaultLargeKnob = MakeShareable(new FSlateDynamicImageBrush(FName(*BrushPath), FVector2D(150.0f, 150.0f)));

			BrushPath = SynthesisPluginBaseDir + "SynthKnobLargeOverlay.png";
			DefaultLargeKnobOverlay = MakeShareable(new FSlateDynamicImageBrush(FName(*BrushPath), FVector2D(150.0f, 150.0f)));

			BrushPath = SynthesisPluginBaseDir + "SynthKnobMedium.png";
			DefaultMediumKnob = MakeShareable(new FSlateDynamicImageBrush(FName(*BrushPath), FVector2D(150.0f, 150.0f)));

			BrushPath = SynthesisPluginBaseDir + "SynthKnobMediumOverlay.png";
			DefaultMediumKnobOverlay = MakeShareable(new FSlateDynamicImageBrush(FName(*BrushPath), FVector2D(150.0f, 150.0f)));

		}
	}
		
	bool bImagesLoaded;

	TSharedPtr<FSlateDynamicImageBrush> DefaultLargeKnob;
	TSharedPtr<FSlateDynamicImageBrush> DefaultLargeKnobOverlay;
	TSharedPtr<FSlateDynamicImageBrush> DefaultMediumKnob;
	TSharedPtr<FSlateDynamicImageBrush> DefaultMediumKnobOverlay;
};

static FSynthKnobResources Resources;

FSynthKnobStyle::FSynthKnobStyle()
	: KnobSize(ESynthKnobSize::Medium)
{
	MinValueAngle = -0.4f;
	MaxValueAngle = 0.4f;

	Resources.LoadImages();

	const FSlateBrush* DefaultMediumBrush = Resources.DefaultMediumKnob.Get();
	MediumKnob = *DefaultMediumBrush;

	const FSlateBrush* DefaultMediumBrushOverlay = Resources.DefaultMediumKnobOverlay.Get();
	MediumKnobOverlay = *DefaultMediumBrushOverlay;

	const FSlateBrush* DefaultLargBrush = Resources.DefaultLargeKnob.Get();
	LargeKnob = *DefaultLargBrush;

	const FSlateBrush* DefaultLargBrushOverlay = Resources.DefaultLargeKnobOverlay.Get();
	LargeKnobOverlay = *DefaultLargBrushOverlay;


}

FSynthKnobStyle::~FSynthKnobStyle()
{
}


void FSynthKnobStyle::Initialize()
{
	//first make sure the style set is setup.  Need to make sure because some things happen before StartupModule
	FSynthSlateStyleSet::Initialize();

	FSynthSlateStyleSet::Get()->Set(FSynthKnobStyle::TypeName, FSynthKnobStyle::GetDefault());
}


const FSlateBrush* FSynthKnobStyle::GetBaseBrush() const
{
	if (KnobSize == ESynthKnobSize::Medium)
	{
		return &MediumKnob;
	}
	else
	{
		return &LargeKnob;
	}

}

const FSlateBrush* FSynthKnobStyle::GetOverlayBrush() const
{
	if (KnobSize == ESynthKnobSize::Medium)
	{
		return &MediumKnobOverlay;
	}
	else
	{
		return &LargeKnobOverlay;
	}
}

void FSynthKnobStyle::GetResources(TArray< const FSlateBrush* >& OutBrushes) const
{
	OutBrushes.Add(&LargeKnob);
	OutBrushes.Add(&LargeKnobOverlay);
	OutBrushes.Add(&MediumKnob);
	OutBrushes.Add(&MediumKnobOverlay);
}

const FSynthKnobStyle& FSynthKnobStyle::GetDefault()
{
	static FSynthKnobStyle Default;
	return Default;
}

const FName FSynthKnobStyle::TypeName( TEXT("SynthKnobStyle") );

