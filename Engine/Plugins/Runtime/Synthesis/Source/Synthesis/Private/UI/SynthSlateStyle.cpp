// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/SynthSlateStyle.h"
#include "Interfaces/IPluginManager.h"
#include "Styling/SlateStyle.h"
#include "SynthesisModule.h"
#include "Internationalization/Regex.h"
#include "HAL/FileManager.h"
#include "Styling/SlateStyleRegistry.h"
#include "Brushes/SlateDynamicImageBrush.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SynthSlateStyle)



TSharedPtr< FSlateStyleSet > FSynthSlateStyleSet::StyleInstance = NULL;

void FSynthSlateStyleSet::Initialize()
{
	if (!StyleInstance.IsValid())
	{
		StyleInstance = MakeShareable(new FSlateStyleSet("SynthesisStyle"));
		StyleInstance->SetContentRoot(IPluginManager::Get().FindPlugin(TEXT("Synthesis"))->GetBaseDir() / TEXT("Resources"));

		FSlateStyleRegistry::RegisterSlateStyle(*StyleInstance);
	}
}

void FSynthSlateStyleSet::Shutdown()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*StyleInstance);
	ensure(StyleInstance.IsUnique());
	StyleInstance.Reset();
}

TSharedPtr< class FSlateStyleSet > FSynthSlateStyleSet::Get()
{
	return StyleInstance; 
}



ISynthSlateResources::ISynthSlateResources()
	: bResourcesLoaded(false)
{
}

ISynthSlateResources::~ISynthSlateResources()
{
}

int32 ISynthSlateResources::GetNumberForImageName(const FString& ImageName)
{
	static const FRegexPattern Digitpattern(TEXT("(?!2x)\\d+"));
	FRegexMatcher Matcher(Digitpattern, ImageName);

	if (Matcher.FindNext())
	{
		int32 Beginning = Matcher.GetMatchBeginning();
		int32 Ending = Matcher.GetMatchEnding();
		FString Number = ImageName.Mid(Beginning, Ending - Beginning);
		return FCString::Atoi(*Number);
	}

	UE_LOGF(LogSynthesis, Error, "Invalid image name for knob png: '%ls'", *ImageName);

	return INDEX_NONE;
}

void ISynthSlateResources::GetImagesAtPath(const FString& DirPath, TArray<TSharedPtr<FSlateDynamicImageBrush>>& OutImages, const float Size)
{
	IFileManager& FileManager = IFileManager::Get();
	TArray<FString> ImageNames;
	FileManager.FindFiles(ImageNames, *DirPath, TEXT(".png"));

	// Sort by priority (lowest priority first).
	ImageNames.Sort([this](const FString& A, const FString& B)
	{
		int32 NumberA = GetNumberForImageName(A);
		int32 NumberB = GetNumberForImageName(B);
		return NumberA < NumberB;
	});

	for (const FString& ImageName : ImageNames)
	{
		TSharedPtr<FSlateDynamicImageBrush> SlateEImageBrush;
		FString BrushPath = DirPath + ImageName;
		const FName BrushName = FName(*BrushPath);
		SlateEImageBrush = MakeShareable(new FSlateDynamicImageBrush(BrushName, FVector2D(Size, Size)));
		OutImages.Add(SlateEImageBrush);
	}
}

ISynthSlateResources* FSynthSlateStyle::SynthSlateResources = nullptr;

FSynthSlateStyle::FSynthSlateStyle()
	: SizeType(ESynthSlateSizeType::Small)
	, ColorStyle(ESynthSlateColorStyle::Light)
{
	if (SynthSlateResources == nullptr)
	{
		SynthSlateResources = CreateSynthSlateResources();
	}

	if (SynthSlateResources)
	{
		SynthSlateResources->LoadResources();
	}
}

FSynthSlateStyle::~FSynthSlateStyle()
{
}

const FSlateBrush* FSynthSlateStyle::GetBrushForValue(const float InValue) const
{
	// Only values in 0.0 to 1.0 are used to look up image from style
	if (InValue < 0.0f || InValue > 1.0f)
	{
		return nullptr;
	}

	if (SynthSlateResources)
	{
		const TArray<TSharedPtr<FSlateDynamicImageBrush>>& Images = SynthSlateResources->GetImagesList(SizeType, ColorStyle);
		int32 ImageIndex = (int32)(InValue * (Images.Num() - 1));
		if (ImageIndex < Images.Num())
		{
			const FSlateBrush* OutBrush = Images[ImageIndex].Get();
			return OutBrush;
		}
	}
	return nullptr;
}

const FSynthSlateStyle& FSynthSlateStyle::GetDefault()
{
	static FSynthSlateStyle Default;
	return Default;
}

const FName FSynthSlateStyle::TypeName( TEXT("FSynthUIStyle") );

