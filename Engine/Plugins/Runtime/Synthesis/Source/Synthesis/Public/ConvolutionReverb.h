// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DSP/AudioChannelFormatConverter.h"
#include "DSP/ConvolutionAlgorithm.h"

#define UE_API SYNTHESIS_API


namespace Audio { class FAlignedBlockBuffer; }

namespace Audio
{
	class FSimpleUpmixer;
	class IFormatConverter;

	/** Gain entry into convolution matrix. */
	struct FConvolutionReverbGainEntry
	{
		int32 InputIndex = 0;

		int32 ImpulseIndex = 0;

		int32 OutputIndex = 0;

		float Gain = 0.f;

		FConvolutionReverbGainEntry(int32 InInputIndex, int32 InImpulseIndex, int32 InOutputIndex, float InGain)
		:	InputIndex(InInputIndex)
		,	ImpulseIndex(InImpulseIndex)
		,	OutputIndex(InOutputIndex)
		,	Gain(InGain)
		{
		}
	};

	/** Runtime settings for convolution reverb. */
	struct FConvolutionReverbSettings
	{
		UE_API FConvolutionReverbSettings();

		/* Used to account for energy added by convolution with "loud" Impulse Responses.  Not meant to be updated dynamically (linear gain)*/
		float NormalizationVolume = -24.f;

		/* Amount of audio to be sent to rear channels in quad/surround configurations (linear gain, < 0 = phase inverted) */
		float RearChannelBleed = 0.f;

		/* If true, send Surround Rear Channel Bleed Amount sends front left to back right and vice versa */
		bool bRearChannelFlip = false;

		static UE_API const FConvolutionReverbSettings DefaultSettings;
	};

	/** Data used to initialize the convolution algorithm */
	struct FConvolutionReverbInitData
	{
		using FInputFormat = IChannelFormatConverter::FInputFormat;
		using FOutputFormat = IChannelFormatConverter::FOutputFormat;

		/* Input audio format. */
		FInputFormat InputAudioFormat;

		/* Output audio format. */
		FOutputFormat OutputAudioFormat;

		/* Algorithm configuration. */
		Audio::FConvolutionSettings AlgorithmSettings;

		/* Sample rate of the impulse response samples. */
		float ImpulseSampleRate = 0.f;

		/* Target sample rate of audio to be processed. */
		float TargetSampleRate = 0.f;

		/* Impulse response samples in interleaved format. */
		TArray<float> Samples;

		/* Normalization gain to apply to reverb output. */
		float NormalizationVolume;

		/* If true, input audio is mixed to match the channel format of the impulse response. 
		 * If false, input audio is matched to equivalent audio channels of the impulse response. 
		 */
		bool bMixInputChannelFormatToImpulseResponseFormat = true;

		/* If true, the reverberated audio is mixed to match the output audio format.
		 * If false, the reverberated audio is matched to the equivalent audio channels of the output audio format. 
		 */
		bool bMixReverbOutputToOutputChannelFormat = true;

		/* If true, impulse response channels are interpreted as true stereo. If
		 * false, impulse response channels are interpreted as individual channel filters.
		 */
		bool bIsImpulseTrueStereo = false;

		/* Array of gain values for convolution algorithm. */
		TArray<FConvolutionReverbGainEntry> GainMatrix;
	};


	/** FConvolutionReverb applies an impulse response to audio.  
	 *
	 * The audio pipeline within an FConvolutionReverb is described in ascii art here.
	 *

				  +-----------+
				  |Input Audio|
				  +-----------+
						|
						v
				  +------------+
				  |Deinterleave|
				  +------------+
						|
						v
		   +----------------------------+
		   |Mix Input Audio to IR Format|
		   +----------------------------+
						|
						v
				+-----------------+
				|Convolve with IRs|
				+-----------------+
						|
						v
		+---------------------------------+
		|Mix Reverb Audio to Output Format|
		+---------------------------------+
						|
						v
				   +----------+
				   |Interleave|
				   +----------+
						|
						v
				  +------------+
				  |Output Audio|
				  +------------+

	*/
	class FConvolutionReverb 
	{
		FConvolutionReverb() = delete;
		FConvolutionReverb(const FConvolutionReverb&) = delete;
		FConvolutionReverb& operator=(const FConvolutionReverb&) = delete;
		FConvolutionReverb(const FConvolutionReverb&&) = delete;


	public:
		~FConvolutionReverb();

		// Create a convolution reverb object. This performs creation of the convolution algorithm object,
		// converting sample rates of impulse responses, sets the impulse response and initializes the
		// gain matrix of the convolution algorithm, and initializes and upmix or downmix objects.
		//
		// @params InInitData - Contains all the information needed to create a convolution reverb.
		// @params InSettings - The initial settings for the convolution reverb.
		//
		// @return TUniquePtr<Audio::IConvolutionAlgorithm>  Will be invalid if there was an error.
		SYNTHESIS_API static TUniquePtr<FConvolutionReverb> CreateConvolutionReverb(const FConvolutionReverbInitData& InInitData, const FConvolutionReverbSettings& InSettings=FConvolutionReverbSettings::DefaultSettings);
		
		SYNTHESIS_API void SetSettings(const FConvolutionReverbSettings& InSettings);

		SYNTHESIS_API const FConvolutionReverbSettings& GetSettings() const;


		// If the number of input frames changes between callbacks, the output may contain discontinuities.
		SYNTHESIS_API void ProcessAudio(int32 InNumInputChannels, const FAlignedFloatBuffer& InputAudio, int32 InNumOutputChannels, FAlignedFloatBuffer& OutputAudio);
		SYNTHESIS_API void ProcessAudio(int32 InNumInputChannels, const float* InputAudio, int32 InNumOutputChannels, float* OutputAudio, const int32 InNumFrames);

		SYNTHESIS_API int32 GetNumInputChannels() const;
		SYNTHESIS_API int32 GetNumOutputChannels() const;

		SYNTHESIS_API static void InterleaveBuffer(FAlignedFloatBuffer& OutBuffer, const TArray<FAlignedFloatBuffer>& InputBuffers, const int32 NumChannels);
		SYNTHESIS_API static void DeinterleaveBuffer(TArray<FAlignedFloatBuffer>& OutputBuffers, TArrayView<const float> InputBuffer, const int32 NumChannels);

	private:

		using FInputFormat = IChannelFormatConverter::FInputFormat;
		using FOutputFormat = IChannelFormatConverter::FOutputFormat;

		// Wraps the various channel converters to provide single interface
		class FChannelFormatConverterWrapper
		{
			public:

				FChannelFormatConverterWrapper() = default;
				FChannelFormatConverterWrapper(TUniquePtr<FSimpleUpmixer> InConverter);
				FChannelFormatConverterWrapper(TUniquePtr<IChannelFormatConverter> InConverter);
				FChannelFormatConverterWrapper(FChannelFormatConverterWrapper&& InOther);
				FChannelFormatConverterWrapper& operator=(FChannelFormatConverterWrapper&& InOther);

				FChannelFormatConverterWrapper(const FChannelFormatConverterWrapper& InOther) = delete;
				FChannelFormatConverterWrapper& operator=(const FChannelFormatConverterWrapper& InOther) = delete;

				bool IsValid() const;

				const FInputFormat& GetInputFormat() const;

				const FOutputFormat& GetOutputFormat() const;
				
				void ProcessAudio(const TArray<FAlignedFloatBuffer>& InInputBuffers, TArray<FAlignedFloatBuffer>& OutOutputBuffers);

				void SetRearChannelBleed(float InGain, bool bFadeToGain=true);

				void SetRearChannelFlip(bool bInDoRearChannelFlip, bool bFadeFlip=true);

				bool GetRearChannelFlip() const;

			private:
				TUniquePtr<IChannelFormatConverter> Storage;

				IChannelFormatConverter* BaseConverter = nullptr;
				FSimpleUpmixer* SimpleUpmixer = nullptr;

				static FInputFormat DefaultInputFormat;
				static FOutputFormat DefaultOutputFormat;
		};


		FConvolutionReverb(TUniquePtr<IConvolutionAlgorithm> InAlgorithm, TUniquePtr<IChannelFormatConverter> InInputConverter, FChannelFormatConverterWrapper&& InOutputConverter, const FConvolutionReverbSettings& InSettings);

		void SetConvolutionAlgorithm(TUniquePtr<IConvolutionAlgorithm> InAlgorithm);

		void ProcessAudioBlock(const float* InputAudio, int32 InNumInputChannels, FAlignedFloatBuffer& OutputAudio, int32 InNumOutputChannels);

		void Update(bool bFadeToParams);

		void ResizeBlockBuffers();

		void ResizeProcessingBuffers();

		void ResizeArrayOfBuffers(TArray<FAlignedFloatBuffer>& InArrayOfBuffers, int32 MinNumBuffers, int32 NumFrames) const;

		FConvolutionReverbSettings Settings;

		TUniquePtr<IConvolutionAlgorithm> ConvolutionAlgorithm;
		TUniquePtr<IChannelFormatConverter> InputChannelFormatConverter;

		FChannelFormatConverterWrapper OutputChannelFormatConverter;
		
		// data is passed to the convolution algorithm as 2D arrays
		TArray<FAlignedFloatBuffer> InputDeinterleaveBuffers;
		TArray<FAlignedFloatBuffer> InputChannelConverterBuffers; 

		// data is recieved from the convolution algorithm as 2D arrays
		TArray<FAlignedFloatBuffer> OutputDeinterleaveBuffers;
		TArray<FAlignedFloatBuffer> OutputChannelConverterBuffers;

		TArray<float*> InputBufferPtrs;
		TArray<float*> OutputBufferPtrs;

		TUniquePtr<FAlignedBlockBuffer> InputBlockBuffer;
		TUniquePtr<FAlignedBlockBuffer> OutputBlockBuffer;
		FAlignedFloatBuffer InterleavedOutputBlock;

		int32 ExpectedNumFramesPerCallback;
		int32 NumInputSamplesPerBlock;
		int32 NumOutputSamplesPerBlock;

		float OutputGain;
		bool bIsConvertingInputChannelFormat;
		bool bIsConvertingOutputChannelFormat;
	};
}

#undef UE_API
