// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "ConvolutionReverb.h"
#include "DSP/Dsp.h"
#include "EffectConvolutionReverb.generated.h"

#define UE_API SYNTHESIS_API

struct FPropertyChangedEvent;

/** Block size of convolution reverb algorithm. */
UENUM()
enum class ESubmixEffectConvolutionReverbBlockSize : uint8
{
	/** 256 audio frames per a block. */
	BlockSize256,

	/** 512 audio frames per a block. */
	BlockSize512,

	/** 1024 audio frames per a block. */
	BlockSize1024
};

#if WITH_EDITORONLY_DATA
DECLARE_MULTICAST_DELEGATE_OneParam(FAudioImpulseResponsePropertyChange, FPropertyChangedEvent&);
#endif

// ========================================================================
// UAudioImpulseResponse
// UAsset used to represent Imported Impulse Responses
// ========================================================================
UCLASS(MinimalAPI, BlueprintType)
class UAudioImpulseResponse : public UObject
{
	GENERATED_BODY()

public:
	UE_API UAudioImpulseResponse();
	/** The interleaved audio samples used in convolution. */
	UPROPERTY()
	TArray<float> ImpulseResponse;

	/** The number of channels in impulse response. */
	UPROPERTY()
	int32 NumChannels;

	/** The original sample rate of the impulse response. */
	UPROPERTY()
	int32 SampleRate;

	/* Used to account for energy added by convolution with "loud" Impulse Responses. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = SubmixEffectPreset, meta = (DisplayName = "Normalization Volume (dB)", ClampMin = "-60.0", ClampMax = "15.0"))
	float NormalizationVolumeDb;

	/* If true, impulse response channels are interpreted as true stereo which allows channel crosstalk. If false, impulse response channels are interpreted as independent channel impulses. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = SubmixEffectPreset, meta = (EditCondition = "(NumChannels > 0) && bIsEvenChannelCount"))
	bool bTrueStereo;

	UPROPERTY(meta = (DeprecatedProperty))
	TArray<float> IRData_DEPRECATED;

	UE_API virtual void Serialize(FArchive& Ar) override;

#if WITH_EDITORONLY_DATA
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	// This delegate is called whenever PostEditChangeProperty called.
	FAudioImpulseResponsePropertyChange OnObjectPropertyChanged;

	// Used to evaluate whether TrueStereo should be editable
	UPROPERTY()
	bool bIsEvenChannelCount;
#endif
};

namespace AudioConvReverbIntrinsics
{
	using namespace Audio;

	typedef int32 FConvolutionReverbID;

	struct FVersionData
	{
		FConvolutionReverbID ConvolutionID;

		FVersionData()
		{
			ConvolutionID = 0;
		}

		bool operator==(const FVersionData& Other) const
		{
			// Check if versions are equal
			return ConvolutionID == Other.ConvolutionID;
		}
	};
}

namespace Audio
{
	class FEffectConvolutionReverb : public TSharedFromThis<FEffectConvolutionReverb, ESPMode::ThreadSafe>
	{
	public:

		UE_API void ProcessAudio(int32 InNumInputChannels, const float* InputAudio, int32 InNumOutputChannels, float* OutputAudio, const int32 InNumFrames);

		UE_API FConvolutionReverbInitData CreateConvolutionReverbInitData();

		// Fire off an async task to build a new reverb algorithm with new settings
		UE_API void BuildReverb();

		// Immediately build the initial reverb algorithm
		UE_API void Init();

		UE_API void SetSettings(const FConvolutionReverbSettings& InSettings);
		UE_API void SetSampleRate(const float InSampleRate);
		UE_API void SetInitData(const FConvolutionReverbInitData& InData);
		UE_API void SetReverb(TUniquePtr<FConvolutionReverb> InReverb);
		UE_API void SetBypass(const bool InBypass);

		UE_API AudioConvReverbIntrinsics::FVersionData UpdateVersion();
		UE_API void UpdateChannelCount(const int32 InNumInputChannels, const int32 InNumOutputChannels);
		UE_API void UpdateParameters();

		UE_API void EnqueueNewReverb(TUniquePtr<FConvolutionReverb> InReverb, const AudioConvReverbIntrinsics::FVersionData& InVersionData);
		UE_API void DequeueNewReverb();

		UE_API bool IsVersionCurrent(const AudioConvReverbIntrinsics::FVersionData& InVersionData) const;
		UE_API bool IsChannelCountUpToDate(const int32 InNumInputChannels, const int32 InNumOutputChannels) const;

	private:
		TUniquePtr<FConvolutionReverb> Reverb;

		// Reverb created by our async builder
		TUniquePtr<FConvolutionReverb> QueuedReverb;

		mutable FCriticalSection VersionDataCriticalSection;
		AudioConvReverbIntrinsics::FVersionData VersionData;
		AudioConvReverbIntrinsics::FVersionData QueuedVersionData;

		mutable FCriticalSection ConvReverbInitDataCriticalSection;
		// Internal copy of data needed to create FConvolutionReverb
		FConvolutionReverbInitData ConvReverbInitData;

		TAtomic<int32> NumInputChannels = 2;
		TAtomic<int32> NumOutputChannels = 2;

		// Params object for copying between audio render thread and outside threads.
		TParams<FConvolutionReverbSettings> Params;

		bool bBypass = false;
		float SampleRate = 0.f;
	};
}

#undef UE_API
