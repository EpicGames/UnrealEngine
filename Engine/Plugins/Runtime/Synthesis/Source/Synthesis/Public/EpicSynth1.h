// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "DSP/Envelope.h"
#include "DSP/Amp.h"
#include "DSP/DelayStereo.h"
#include "DSP/Chorus.h"
#include "EpicSynth1Types.h"

namespace Audio
{
	class FEpicSynth1;

	class FEpicSynth1Voice
	{
	public:
		static const int32 NumOscillators = 2;
		static const int32 NumLFOs = 2;

		FEpicSynth1Voice();
		~FEpicSynth1Voice();

		void Init(FEpicSynth1* InParentSynth, const int32 InVoiceId);
		void Reset();

		void NoteOn(const uint32 InMidiNote, const float InVelocity, const float InDurationSec);
		void NoteOff(const uint32 InMidiNote, const bool bAllNotesOff = false);
		void Kill();
		void Shutdown();

		bool IsFinished() const { return bIsFinished; }
		bool IsActive() const { return bIsActive; }

		uint32 GetGeneration() const { return VoiceGeneration; }
		int32 GetMidiNote() const { return MidiNote; }
		void SetLFOPatch(const int32 InLFOIndex, const ESynthLFOPatchType InPatchType);
		void SetEnvModPatch(const ESynthModEnvPatch InPatchType);
		void SetEnvModBiasPatch(const ESynthModEnvBiasPatch InPatchType);

		void Generate(float InSamples[2]);

		void ClearPatches();
		bool CreatePatch(const FPatchId PatchId, const ESynth1PatchSource PatchSource, const TArray<FSynth1PatchCable>& PatchCables, const bool bEnableByDefault);
		bool SetEnablePatch(const FPatchId PatchId, const bool bIsEnabled);

	protected:

		FPatchSource GetPatchSource(ESynth1PatchSource PatchSource);
		void GetPatchDestinations(ESynth1PatchDestination PatchDestination, TArray<FPatchDestination>& Destinations);

		// Audio rate objects
		FOsc Oscil[NumOscillators];
		FAmp OscilPan[NumOscillators];
		FAmp Amp;
		FOnePoleFilter OnePoleFilter;
		FStateVariableFilter StateVarFilter;
		FLadderFilter LadderFilter;
		// Which filter we're currently using
		IFilter* CurrentFilter;

		// Control rate objects
		FEnvelope GainEnv;
		FEnvelope ModEnv;
		FLFO LFO[NumLFOs];
		FLinearEase PortamentoFrequency;

		// Mod-matrix patches for the voice
		FPatch LFOPatches[NumLFOs][(int32)ESynthLFOPatchType::Count];
		FPatch ModEnvPatches[(int32)ESynthModEnvPatch::Count];
		FPatch ModEnvBiasPatches[(int32)ESynthModEnvBiasPatch::Count];

		TMap<int32, TSharedPtr<FPatch>> DynamicPatches;

		ESynthLFOPatchType CurrentPatchType[NumLFOs];
		ESynthModEnvPatch CurrentModPatchType;
		ESynthModEnvBiasPatch CurrentModBiasPatchType;

		FPatch Env_To_Amp;
		FPatch Env_To_Filter;

		// Data.
		int32 MidiNote;
		int32 VoiceId;
		int32 ControlSampleCount;
		int32 DurationSampleCount;
		int32 CurrentSampleCount;

		// Used to do voice stealing
		uint32 VoiceGeneration;

		// Owning synth
		FEpicSynth1* ParentSynth;

		// If voice has finished
		bool bIsFinished;

		// If voice is active (i.e. not free)
		bool bIsActive;

		friend class FEpicSynth1;
	};

	class FEpicSynth1
	{
	public:
		FEpicSynth1();
		virtual ~FEpicSynth1();

		void Init(const float InSampleRate, const int32 InMaxVoices);

		// Set the synth to be in mono mode
		void SetMonoMode(const bool bIsMono);

		// Plays a note with the given midi note and midi velocity
		void NoteOn(const uint32 InMidiNote, const float InVelocity, const float Duration = -1.0f);

		// Turns the note off (triggers release envelope)
		void NoteOff(const uint32 InMidiNote, const bool bAllNotesOff = false, const bool bKillAllNotes = false);

		/// Oscillator setting
		void SetOscType(const int32 InOscIndex, const EOsc::Type InOscType);
		void SetOscGain(const int32 InOscIndex, const float InGain);
		void SetOscGainMod(const int32 InOscIndex, const float InGain);
		void SetOscDetune(const int32 InOscIndex, const float InDetuneFreq);
		void SetOscOctave(const int32 InOscIndex, const float InOctave);
		void SetOscSemitones(const int32 InOscIndex, const float InSemitones);
		void SetOscCents(const int32 InOscIndex, const float InCents);
		void SetOscPitchBend(const int32 InOscIndex, const float InPitchBend);
		void SetOscPulseWidth(const int32 InOscIndex, const float InPulseWidth);
		void SetOscPortamento(const float InPortamento);
		void SetOscSpread(const float InSpread);
		void SetOscUnison(const bool bInUnison);
		void SetOscSync(const bool bIsSync);

		/// Filter setting
		void SetFilterAlgorithm(const ESynthFilterAlgorithm FilterAlgorithm);
		void SetFilterType(const EFilter::Type FilterType);
		void SetFilterFrequency(const float InFilterFrequency);
		void SetFilterFrequencyMod(const float InFilterFrequencyModulation);
		void SetFilterQ(const float InFilterQ);
		void SetFilterQMod(const float InFilterQModulation);

		/// LFO setting
		void SetLFOType(const int32 LFOIndex, const ELFO::Type InLFOType);	
		void SetLFOMode(const int32 LFOIndex, const ELFOMode::Type InLFOMode);
		void SetLFOPatch(const int32 LFOIndex, const ESynthLFOPatchType InRouteType);
		void SetLFOGain(const int32 LFOIndex, const float InLFOGain);
		void SetLFOGainMod(const int32 LFOIndex, const float InLFOGainMod);
		void SetLFOFrequency(const int32 LFOIndex, const float InLFOFrequency);
		void SetLFOFrequencyMod(const int32 LFOIndex, const float InLFOFrequencyMod);
		void SetLFOPulseWidth(const int32 LFOIndex, const float InPulseWidth);

		/// Envelope setting
		void SetEnvAttackTime(const float InAttackTimeMsec);
		void SetEnvDecayTime(const float InDecayTimeMsec);
		void SetEnvSustainGain(const float InSustainGain);
		void SetEnvReleaseTime(const float InReleaseTimeMsec);
		void SetEnvLegatoEnabled(const bool bIsLegatoEnabled);
		void SetEnvRetriggerMode(const bool bIsRetriggerMode);

		/// Modulation Envelope Setting
		void SetModEnvPatch(const ESynthModEnvPatch InPatchType);
		void SetModEnvBiasPatch(const ESynthModEnvBiasPatch InPatchType);
		void SetModEnvInvert(const bool bInInvert);
		void SetModEnvBiasInvert(const bool bInInvert);
		void SetModEnvDepth(const float InDepth);
		void SetModEnvAttackTime(const float InAttackTimeMsec);
		void SetModEnvDecayTime(const float InDecayTimeMsec);
		void SetModEnvSustainGain(const float InSustainGain);
		void SetModEnvReleaseTime(const float InReleaseTimeMsec);

		/// Pan and gain setting
		void SetPan(const float InPan);
		void SetGainDb(const float InGainDb);

		/// Stereo Delay parameters
		void SetStereoDelayIsEnabled(const bool bInIsStereoEnabled);
		void SetStereoDelayMode(EStereoDelayMode::Type InStereoDelayMode);
		void SetStereoDelayTimeMsec(const float InDelayTimeMsec);
		void SetStereoDelayFeedback(const float InDelayFeedback);
		void SetStereoDelayRatio(const float InDelayRatio);
		void SetStereoDelayWetLevel(const float InDelayWetLevel);
		
		/// Chorus parameters
		void SetChorusEnabled(const bool bInIsChorusEnabled);
		void SetChorusDepth(const EChorusDelays::Type InType, const float InDepth);
		void SetChorusFeedback(const EChorusDelays::Type InType, const float InFeedback);
		void SetChorusFrequency(const EChorusDelays::Type InType, const float InFrequency);

		void ClearPatches();
		FPatchId CreatePatch(const ESynth1PatchSource PatchSource, const TArray<FSynth1PatchCable>& PatchCables, const bool bEnableByDefault);
		bool SetEnablePatch(const FPatchId PatchId, bool bIsEnabled);

		// Generate the next frame of audio
		void GenerateFrame(float* OutFrame);

	protected:

		void SwitchFilter();
		uint32 GetOldestPlayingId();
		void StopAllVoicesExceptNewest();

		// The max number of voices allowed in the synth
		int32 MaxNumVoices;

		// The current number of voices, may be less than MaxNumVoices
		int32 NumVoices;

		// Count of number of voices in flight. Should not be greater than NumVoices.
		int32 NumActiveVoices;

		// Number of buffer voices used for stopping voices
		int32 NumStoppingVoices;

		// The last played voice
		FEpicSynth1Voice* LastVoice;

		// Sample rate of the synth
		float SampleRate;

		// Control sample rate (LFOs, etc)
		float ControlSampleRate;

		// The number of real samples per control rate tick
		int32 ControlSamplePeriod;

		// Mod matrix object (used to route connections)
		FModulationMatrix ModMatrix;

		// Time to pitch shift up or down to target notes
		float Portamento;

		// Last midi note played, used for portamento
		uint32 LastMidiNote;

		// The allocated voice pool
		TArray<FEpicSynth1Voice*> Voices;

		// List of free voice indices
		TArray<int32> FreeVoices;

		// An incremented number used to track voice age. Older voices have smaller generation counts.
		uint32 VoiceGeneration;

		// Filter data
		float BaseFilterFreq;
		float FilterFreqMod;

		float BaseFilterQ;
		float FilterQMod;

		EFilter::Type FilterType;
		ESynthFilterAlgorithm FilterAlgorithm;

		// Stereo delay effect
		FDelayStereo StereoDelay;
		FChorus Chorus;
		friend class FEpicSynth1Voice;

		bool bIsUnison : 1;
		bool bIsStereoEnabled : 1;
		bool bIsChorusEnabled : 1;
	};

}
