// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EpicSynth1Types.generated.h"

UENUM(BlueprintType)
enum class ESynth1OscType : uint8
{
	Sine = 0,
	Saw,
	Triangle,
	Square,
	Noise,
	Count UMETA(Hidden)
};

UENUM(BlueprintType)
enum class ESynthLFOType : uint8
{
	Sine = 0,
	UpSaw,
	DownSaw,
	Square,
	Triangle,
	Exponential,
	RandomSampleHold,
	Count UMETA(Hidden)
};

UENUM(BlueprintType)
enum class ESynthLFOMode : uint8
{
	Sync = 0,
	OneShot,
	Free,
	Count UMETA(Hidden)
};

UENUM(BlueprintType)
enum class ESynthLFOPatchType : uint8
{
	PatchToNone = 0,

	PatchToGain,
	PatchToOscFreq,
	PatchToFilterFreq,
	PatchToFilterQ,
	PatchToOscPulseWidth,
	PatchToOscPan,
	PatchLFO1ToLFO2Frequency,
	PatchLFO1ToLFO2Gain,
	Count UMETA(Hidden)
};

UENUM(BlueprintType)
enum class ESynthModEnvPatch : uint8
{
	PatchToNone = 0,

	PatchToOscFreq,
	PatchToFilterFreq,
	PatchToFilterQ,
	PatchToLFO1Gain,
	PatchToLFO2Gain,
	PatchToLFO1Freq,
	PatchToLFO2Freq,
	Count UMETA(Hidden)
};

UENUM(BlueprintType)
enum class ESynthModEnvBiasPatch : uint8
{
	PatchToNone = 0,

	PatchToOscFreq,
	PatchToFilterFreq,
	PatchToFilterQ,
	PatchToLFO1Gain,
	PatchToLFO2Gain,
	PatchToLFO1Freq,
	PatchToLFO2Freq,
	Count UMETA(Hidden)
};

UENUM(BlueprintType)
enum class ESynthFilterType : uint8
{
	LowPass = 0,
	HighPass,
	BandPass,
	BandStop,
	Count UMETA(Hidden)
};

UENUM(BlueprintType)
enum class ESynthFilterAlgorithm : uint8
{
	OnePole = 0,
	StateVariable,
	Ladder,
	Count UMETA(Hidden)
};

UENUM(BlueprintType)
enum class ESynthStereoDelayMode : uint8
{
	Normal = 0,
	Cross,
	PingPong,
	Count UMETA(Hidden)
};

UENUM(BlueprintType)
enum class ESynth1PatchSource : uint8
{
	LFO1,
	LFO2,
	Envelope,
	BiasEnvelope,
	Count UMETA(Hidden)
};

UENUM(BlueprintType)
enum class ESynth1PatchDestination : uint8
{
	Osc1Gain,
	Osc1Frequency,
	Osc1Pulsewidth,
	Osc2Gain,
	Osc2Frequency,
	Osc2Pulsewidth,
	FilterFrequency,
	FilterQ,
	Gain,
	Pan,
	LFO1Frequency,
	LFO1Gain,
	LFO2Frequency,
	LFO2Gain,
	Count UMETA(Hidden)
};

USTRUCT(BlueprintType)
struct FSynth1PatchCable
{
	GENERATED_USTRUCT_BODY()

	// The patch depth (how much the modulator modulates the destination)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset")
	float Depth = 0.0f;

	// The patch destination type
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Synth|Preset")
	ESynth1PatchDestination Destination = ESynth1PatchDestination::Osc1Gain;
};

USTRUCT(BlueprintType)
struct FPatchId
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	int32 Id = INDEX_NONE;
};
