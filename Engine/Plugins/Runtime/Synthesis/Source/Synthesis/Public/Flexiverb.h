// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "DSP/Delay.h"

namespace Audio { class FDelayAPF; }
namespace Audio { class FOnePoleLPF; }

namespace Audio
{
	struct FFlexiverbSettings
	{
		// The higher the complexity value, the higher the computational and memory requirements.
		// However, increasing the complexity value will also increase the echo density of the reverb.
		// Note that setting a new complexity value will induce reallocation of memory.
		float Complexity;

		//An approximation of the time, in seconds, the reverb should take to decay to -60 dB.
		float DecayTime;

		//The amount, in milliseconds, of delay before the reverberation effect begins.
		//This is equivalent to the hearing the first reflection of a sound source in a room.
		float PreDelay;

		//Frequency, in HZ. All frequencies above this value are attenuated.
		float RoomDampening;

		FFlexiverbSettings()
			: Complexity(4)
			, DecayTime(6.0f)
			, PreDelay(6.0f)
			, RoomDampening(220.0f)
		{
		}
	};

	/********************************************************************************/
	/* FFlexiverb														         	*/
	/* Flexiverb is a computationally inexpensive single-channel reverb algorithm   */
	/* optimized for maximizing echo density with as few multiplies as possible.    */
	/* This effect is best suited for emulating small room reverberation on         */
	/* platforms where CPU resources are limited.                                   */
	/* At longer decay times, this reverb algorithm begins to sound very metallic.  */
	/* This algorithm uses a Householder matrix as the scattering matrix for an FDN.*/
	/* More information on this can be found here:                                  */
	/* https://ccrma.stanford.edu/~jos/pasp/Householder_Feedback_Matrix.html        */
	/********************************************************************************/

	class FFlexiverb
	{
	public:
		SYNTHESIS_API FFlexiverb();
		SYNTHESIS_API ~FFlexiverb();

		// Initialize the reverb with the given sample rate and initial settings.
		SYNTHESIS_API void Init(const int32 InSampleRate, const FFlexiverbSettings Settings = FFlexiverbSettings());

		// Updates the current settings of this reverb.
		SYNTHESIS_API void SetSettings(const FFlexiverbSettings& InSettings);

		// Process a single audio frame.
		SYNTHESIS_API void ProcessAudioFrame(const float* InBuffer, const int32 InChannels, float* OutBuffer, const int32 OutChannels);

	private:
		// Calculates coefficients based on the current settings.
		void Update();

		// Manage memory associated with our FDN.
		void UpdateComplexity(const int32 Complexity);

		// Sample rate this instance is initialized to.
		float SampleRate;

		//Private copy of the current reverb settings.
		FFlexiverbSettings PrivateSettings;

		//Delay Line used for pre-delay.
		FDelay PreDelayLine;

		//Delay lines used for the primary FDN.
		TArray<FDelayAPF> DelayLines;

		//Series of APF filters used to decorrelate the input before it enters the FDN.
		TArray<FDelayAPF> APFArray;

		//Room dampening is handled with second order parametric filters.
		TArray<FOnePoleLPF> DampeningArray;

		//Factor applied to the input of all delays lines.
		float DecayFactor;

		//Amount of delay lines in our FDN.
		int32 ScatteringMatrixLength;
	};
}
