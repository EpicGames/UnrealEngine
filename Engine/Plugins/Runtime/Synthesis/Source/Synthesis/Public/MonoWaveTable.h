// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DSP/Filter.h"
#include "DSP/LFO.h"
#include "DSP/Envelope.h"
#include "SampleBuffer.h"

namespace Audio
{
	typedef int32 DefaultWaveTableIndexType;

	class FMonoWaveTable
	{
	public:
		FMonoWaveTable();
		~FMonoWaveTable();

		void Init(int32& InSampleRate, DefaultWaveTableIndexType InNumTableElements, float InWaveTableResolution);
		void NoteOn(float InFrequency, float Velocity = 1.0f);
		void NoteOff(float InFrequency);

		/// Oscillator/Table setting
		float GetWaveTableResolution() { return WaveTableResolution; }
		void UpdateWaveTable(DefaultWaveTableIndexType Index, const TArray<float>& InTable);
		void SetFrequency(float InFrequencyHz) { FrequencyInHz = InFrequencyHz; }
		void SetFrequencyOffset(float InFrequencyCents) { PitchOffsetCents = InFrequencyCents; }
		void SetPosition(float InPosition) { WaveTablePosition = FMath::Clamp(InPosition, 0.0f, 1.0f); }

		/// Filter setting
		void SetLpfFreq(float InNewFrequency) { LPFBaseFreqHz = InNewFrequency; }
		void SetLpfRes(float InNewQ) { LPF.SetQ(InNewQ); }

		/// LFO setting
		// Table Position
		void SetPosLFOFrequency(const float InPosLFOFrequency);
		void SetPosLFODepth(const float InPosLFODepth);
		void SetPosLFOType(const ELFO::Type InPosLFOType);

		/// Envelope setting
		// Amp
		void SetAmpEnvelopeAttackTime(const float InAttackTimeMsec);
		void SetAmpEnvelopeDecayTime(const float InDecayTimeMsec);
		void SetAmpEnvelopeSustainGain(const float InSustainGain);
		void SetAmpEnvelopeReleaseTime(const float InReleaseTimeMsec);
		void SetAmpEnvelopeInvert(const bool bInInvert);
		void SetAmpEnvelopeBiasInvert(const bool bInBiasInvert);
		void SetAmpEnvelopeDepth(const float InDepth);
		void SetAmpEnvelopeBiasDepth(const float InDepth);

		// Filter
		void SetFilterEnvelopeAttackTime(const float InAttackTimeMsec);
		void SetFilterEnvelopenDecayTime(const float InDecayTimeMsec);
		void SetFilterEnvelopeSustainGain(const float InSustainGain);
		void SetFilterEnvelopeReleaseTime(const float InReleaseTimeMsec);
		void SetFilterEnvelopeInvert(const bool bInInvert);
		void SetFilterEnvelopeBiasInvert(const bool bInBiasInvert);
		void SetFilterEnvelopeDepth(const float InDepth);
		void SetFilterEnvelopeBiasDepth(const float InDepth);

		// Table Position
		void SetPositionEnvelopeAttackTime(const float InAttackTimeMsec);
		void SetPositionEnvelopeDecayTime(const float InDecayTimeMsec);
		void SetPositionEnvelopeSustainGain(const float InSustainGain);
		void SetPositionEnvelopeReleaseTime(const float InReleaseTimeMsec);
		void SetPositionEnvelopeInvert(const bool bInInvert);
		void SetPositionEnvelopeBiasInvert(const bool bInBiasInvert);
		void SetPositionEnvelopeDepth(const float InDepth);
		void SetPositionEnvelopeBiasDepth(const float InDepth);

		/// Performance
		void SetSustainPedalPressed(bool bInSustainPedalState);


		// Generate the next frame of audio
		DefaultWaveTableIndexType OnGenerateAudio(float* OutAudio, DefaultWaveTableIndexType NumSamples);

	protected:
		// The number of samples in a wave table entry
		float WaveTableResolution;

		// Audio rendering Sample rate
		float OutputSampleRate;

		// frequency of current note (mono)
		float FrequencyInHz;

		// Index into Wave Table
		float Phase;

		// Current base index into wave table (modulators contribute to final position)
		// (float between [0.0, 1.0]; 0.0 = first wave table, 1.0 = last wave table)
		float WaveTablePosition;

		float LastWaveTablePosition;

		// The current base frequency of LPF (modulators contribute to final value)
		float LPFBaseFreqHz { 1'000.0f };
		float LPFMaxModOffsetOctaves;
		float LPFCeilingHz;
		float LastLPFFreqHz;
		float NewAmpSustain { 0.0f };
		float LastAmpSustain { 0.0f };

		float PitchOffsetCents; // Pitch bend wheel

		// Table
		//TArray<TArray<float>> WaveTableMatrix;
		TArray<TSampleBuffer<float>> WaveTableMatrix;
		TArray<float> WaveTableLookupOutput;

		// control data
		Audio::FLFO PosLfo;
		Audio::FEnvelope AmpEnv;
		Audio::FEnvelope LPFEnv;
		Audio::FEnvelope PosEnv;

		// Filters
		Audio::FLadderFilter LPF;
	
		// helper that performs fractional indexing and wraps TableIndex
		float UpdateWaveTableLookupOutput(float InPhaseIndex);
		float GetMonoSynthOutput(float InTablePosition);

		// frequencies of unreleased notes
		TArray<float> NoteStack;

		uint8 bInitialized : 1;
		uint8 bIsPlaying : 1;
		uint8 bIsSustainPedalPressed : 1;
	};
} // namespace Audio
