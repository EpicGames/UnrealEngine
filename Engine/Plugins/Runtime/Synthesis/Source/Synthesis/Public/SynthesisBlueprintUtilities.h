// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "SynthesisBlueprintUtilities.generated.h"

/** Synthesis Utilities Blueprint Function Library
*  A library of synthesis related functions for use in Blueprints
*/
UCLASS(MinimalAPI)
class USynthesisUtilitiesBlueprintFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

	// Returns the log frequency of the input value. Maps linear domain and range values to log output (good for linear slider controlling frequency)
	UFUNCTION(BlueprintCallable, Category = "Synthesis Utilities Library")
	static float GetLogFrequency(float InLinearValue, float InDomainMin, float InDomainMax, float InRangeMin, float InRangeMax);

	// Returns the log frequency of the input value. Maps linear domain and range values to log output (good for linear slider controlling frequency)
	UFUNCTION(BlueprintCallable, Category = "Synthesis Utilities Library")
	static float GetLinearFrequency(float InLogFrequencyValue, float InDomainMin, float InDomainMax, float InRangeMin, float InRangeMax);

};
