// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateWidgetStyleAsset.h"
#include "UI/SynthSlateStyle.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Framework/SlateDelegates.h"
#include "Widgets/SLeafWidget.h"
#include "Styling/SlateStyle.h"
#include "UI/Synth2DSliderStyle.h"

#define UE_API SYNTHESIS_API

class FPaintArgs;
class FSlateWindowElementList;
struct FSynth2DSliderStyle;

/**
* A Slate slider control is a linear scale and draggable handle.
*/
class SSynth2DSlider : public SLeafWidget
{
public:

	SLATE_BEGIN_ARGS(SSynth2DSlider)
		: _IndentHandle(true)
		, _Locked(false)
		, _Style(&FSynthSlateStyleSet::Get()->GetWidgetStyle<FSynth2DSliderStyle>("Synth2DSliderStyle"))
		, _StepSize(0.01f)
		, _ValueX(1.f)
		, _ValueY(1.f)
		, _IsFocusable(true)
		, _OnMouseCaptureBegin()
		, _OnMouseCaptureEnd()
		, _OnValueChangedX()
		, _OnValueChangedY()
	{ }

	/** Whether the slidable area should be indented to fit the handle. */
	SLATE_ATTRIBUTE(bool, IndentHandle)

		/** Whether the handle is interactive or fixed. */
		SLATE_ATTRIBUTE(bool, Locked)

		/** The style used to draw the slider. */
		SLATE_STYLE_ARGUMENT(FSynth2DSliderStyle, Style)

		/** The input mode while using the controller. */
		SLATE_ATTRIBUTE(float, StepSize)

		/** A value that drives where the slider handle appears. Value is normalized between 0 and 1. */
		SLATE_ATTRIBUTE(float, ValueX)

		/** A value that drives where the slider handle appears. Value is normalized between 0 and 1. */
		SLATE_ATTRIBUTE(float, ValueY)

		/** Sometimes a slider should only be mouse-clickable and never keyboard focusable. */
		SLATE_ARGUMENT(bool, IsFocusable)

		/** Invoked when the mouse is pressed and a capture begins. */
		SLATE_EVENT(FSimpleDelegate, OnMouseCaptureBegin)

		/** Invoked when the mouse is released and a capture ends. */
		SLATE_EVENT(FSimpleDelegate, OnMouseCaptureEnd)

		/** Invoked when the Controller is pressed and capture begins. */
		SLATE_EVENT(FSimpleDelegate, OnControllerCaptureBegin)

		/** Invoked when the controller capture is released.  */
		SLATE_EVENT(FSimpleDelegate, OnControllerCaptureEnd)

		/** Called when the value is changed by the slider. */
		SLATE_EVENT(FOnFloatValueChanged, OnValueChangedX)

		/** Called when the value is changed by the slider. */
		SLATE_EVENT(FOnFloatValueChanged, OnValueChangedY)

		SLATE_END_ARGS()

		/**
		* Construct the widget.
		*
		* @param InDeclaration A declaration from which to construct the widget.
		*/
		UE_API void Construct(const SSynth2DSlider::FArguments& InDeclaration);

	/** See the Value attribute */
	UE_API float GetValueX() const;
	UE_API float GetValueY() const;

	/** See the Value attribute */
	UE_API void SetValueX(const TAttribute<float>& InValueAttribute);
	UE_API void SetValueY(const TAttribute<float>& InValueAttribute);

	/** See the IndentHandle attribute */
	UE_API void SetIndentHandle(const TAttribute<bool>& InIndentHandle);

	/** See the Locked attribute */
	UE_API void SetLocked(const TAttribute<bool>& InLocked);

	/** See the Orientation attribute */
	UE_API void SetOrientation(EOrientation InOrientation);

	/** See the SliderBarColor attribute */
	UE_API void SetSliderBarColor(FSlateColor InSliderBarColor);

	/** See the SliderHandleColor attribute */
	UE_API void SetSliderHandleColor(FSlateColor InSliderHandleColor);

	/** See the StepSize attribute */
	UE_API void SetStepSize(const TAttribute<float>& InStepSize);

public:

	// SWidget overrides

	UE_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	UE_API virtual FVector2D ComputeDesiredSize(float) const override;
	UE_API virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	UE_API virtual FReply OnKeyUp(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	UE_API virtual void OnFocusLost(const FFocusEvent& InFocusEvent) override;

	UE_API virtual bool SupportsKeyboardFocus() const override;
	UE_API virtual bool IsInteractable() const override;

	/** @return Is the handle locked or not? Defaults to false */
	UE_API bool IsLocked() const;

protected:

	/**
	* Commits the specified slider value.
	*
	* @param NewValue The value to commit.
	*/
	UE_API void CommitValue(float NewValueX, float NewValueY);

	/**
	* Calculates the new value based on the given absolute coordinates.
	*
	* @param MyGeometry The slider's geometry.
	* @param AbsolutePosition The absolute position of the slider.
	* @return The new value.
	*/
	UE_API FVector2D PositionToValue(const FGeometry& MyGeometry, const FVector2D& AbsolutePosition);

private:

	// Holds the style passed to the widget upon construction.
	const FSynth2DSliderStyle* Style;

	// Holds a flag indicating whether the slideable area should be indented to fit the handle.
	TAttribute<bool> IndentHandle;

	// Holds a flag indicating whether the slider is locked.
	TAttribute<bool> LockedAttribute;

	// Holds the slider's orientation.
	EOrientation Orientation;

	// Holds the color of the slider bar.
	TAttribute<FSlateColor> SliderBarColor;

	// Holds the color of the slider handle.
	TAttribute<FSlateColor> SliderHandleColor;

	TAttribute<float> ValueAttributeX;
	TAttribute<float> ValueAttributeY;

	/** Holds the amount to adjust the value by when using a controller or keyboard */
	TAttribute<float> StepSize;

	// Holds a flag indicating whether a controller/keyboard is manipulating the slider's value. 
	// When true, navigation away from the widget is prevented until a new value has been accepted or canceled. 
	bool bControllerInputCaptured;

	/** When true, this slider will be keyboard focusable. Defaults to false. */
	bool bIsFocusable;

private:

	// Resets controller input state. Fires delegates.
	void ResetControllerState();

	// Holds a delegate that is executed when the mouse is pressed and a capture begins.
	FSimpleDelegate OnMouseCaptureBegin;

	// Holds a delegate that is executed when the mouse is let up and a capture ends.
	FSimpleDelegate OnMouseCaptureEnd;

	// Holds a delegate that is executed when capture begins for controller or keyboard.
	FSimpleDelegate OnControllerCaptureBegin;

	// Holds a delegate that is executed when capture ends for controller or keyboard.
	FSimpleDelegate OnControllerCaptureEnd;

	// Holds a delegate that is executed when the slider's value changed.
	FOnFloatValueChanged OnValueChangedX;
	FOnFloatValueChanged OnValueChangedY;
};

#undef UE_API
