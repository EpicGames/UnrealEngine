// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "UI/SynthSlateStyle.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Framework/SlateDelegates.h"
#include "Widgets/SLeafWidget.h"
#include "UI/SynthKnobStyle.h"

class FPaintArgs;
class FSlateWindowElementList;
struct FSynthKnobStyle;

class SSynthKnob : public SLeafWidget
{
public:
	
	SLATE_BEGIN_ARGS(SSynthKnob)
		: _Locked(false)
		, _Style(&FSynthSlateStyleSet::Get()->GetWidgetStyle<FSynthKnobStyle>("SynthKnobStyle"))
		, _StepSize(0.01f)
		, _MouseSpeed(1.0f)
		, _MouseFineTuneSpeed(0.2f)
		, _Value(0.5f)
		, _ParameterName(FText::GetEmpty())
		, _ParameterUnits(FText::GetEmpty())
		, _ParameterRange(FVector2D(0.0f, 1.0f))
		, _ShowParamTooltip(true)
		, _IsFocusable(true)
		, _OnMouseCaptureBegin()
		, _OnMouseCaptureEnd()
		, _OnValueChanged()
		{}

		/** Whether the handle is interactive or fixed. */
		SLATE_ATTRIBUTE(bool, Locked)

		/** The style used to draw the knob. */
		SLATE_STYLE_ARGUMENT(FSynthKnobStyle, Style)

		/** The speed of the knob rotation. 1.0 is default speed, 2.0 is 2x as fast as default, 0.5 is half as fast as default speed. */
		SLATE_ATTRIBUTE(float, StepSize)

		/** The speed of the knob rotation with the mouse. 1.0 is default speed, 2.0 is 2x as fast as default, 0.5 is half as fast as default speed. */
		SLATE_ATTRIBUTE(float, MouseSpeed)

		/** The speed of the knob rotation with the mouse when shift is held while using mouse. */
		SLATE_ATTRIBUTE(float, MouseFineTuneSpeed)

		/** A value that drives where the slider handle appears. Value is normalized between 0 and 1. */
		SLATE_ATTRIBUTE(float, Value)

		/** The name of the parameter. Can be queried and will show in a popup. */
		SLATE_ATTRIBUTE(FText, ParameterName)

		/** The parameter units. */
		SLATE_ATTRIBUTE(FText, ParameterUnits)

		/** Parameter value range. */
		SLATE_ATTRIBUTE(FVector2D, ParameterRange)

		/** Sometimes a slider should only be mouse-clickable and never keyboard focusable. */
		SLATE_ARGUMENT(bool, ShowParamTooltip)

		/** Sometimes a slider should only be mouse-clickable and never keyboard focusable. */
		SLATE_ARGUMENT(bool, IsFocusable)

		/** Invoked when the mouse is pressed and a capture begins. */
		SLATE_EVENT(FSimpleDelegate, OnMouseCaptureBegin)

		/** Invoked when the mouse is released and a capture ends. */
		SLATE_EVENT(FSimpleDelegate, OnMouseCaptureEnd)

		/** Invoked when the Controller is pressed and capture begins. */
		SLATE_EVENT(FSimpleDelegate, OnControllerCaptureBegin)

		/** Invoked when the controller capture is released.  */
		SLATE_EVENT(FSimpleDelegate, OnControllerCaptureEnd)

		/** Called when the value is changed by the slider. */
		SLATE_EVENT(FOnFloatValueChanged, OnValueChanged)

	SLATE_END_ARGS()

		/**
	 * Construct the widget.
	 * 
	 * @param InDeclaration A declaration from which to construct the widget.
	 */
	void Construct( const SSynthKnob::FArguments& InDeclaration );

	/** See the Value attribute */
	float GetValue() const;

	/** See the Value attribute */
	void SetValue(const TAttribute<float>& InValueAttribute);

	/** See the Locked attribute */
	void SetLocked(const TAttribute<bool>& InLocked);

	/** See the StepSize attribute */
	void SetStepSize(const float InStepSize);

	/** See the MousePeed attribute */
	void SetMouseSpeed(const float InMouseSpeed);

	/** See the MousePeed attribute */
	void SetMouseFineTuneSpeed(const float InMouseFineTuneSpeed);

	/** Return the last mouse down position. */
	FVector2D GetMouseDownPosition();

public:

	// SWidget overrides

	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	virtual FVector2D ComputeDesiredSize(float LayoutScaleMultiplier) const override;
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	virtual FReply OnKeyUp(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	virtual void OnFocusLost(const FFocusEvent& InFocusEvent) override;
	virtual bool SupportsKeyboardFocus() const override;
	virtual bool IsInteractable() const override;

	/** @return Is the handle locked or not? Defaults to false */
	bool IsLocked() const;
		/**
	 * Commits the specified slider value.
	 *
	 * @param NewValue The value to commit.
	 */
	void CommitValue(float NewValue);

private:

	// Holds the style passed to the widget upon construction.
	const FSynthKnobStyle* Style;

	// Holds a flag indicating whether the slider is locked.
	TAttribute<bool> LockedAttribute;

	// Holds the slider's orientation.
	EOrientation Orientation;

	// Holds the slider's current value.
	TAttribute<float> ValueAttribute;

	/** Holds the amount to adjust the value by when using a controller or keyboard */
	TAttribute<float> StepSize;

	/** Holds the amount to adjust the value by when using a controller or keyboard */
	TAttribute<float> MouseSpeed;

	/** Holds the amount to adjust the value by when using a controller or keyboard */
	TAttribute<float> MouseFineTuneSpeed;

	TAttribute<FText> ParameterName;
	TAttribute<FText> ParameterUnits;
	TAttribute<FVector2D> ParameterRange;
	TAttribute<bool> ShowTooltip;

	// The position of the mouse when it pushed down and started rotating the knob
	FVector2D MouseDownPosition;

	// the value when the mouse was pushed down
	float MouseDownValue;

	// the max pixels to go to min or max value (clamped to 0 or 1) in one drag period
	int32 PixelDelta;

	// The key to use when fine tuning
	FKey FineTuneKey;

	// Holds a flag indicating whether a controller/keyboard is manipulating the slider's value. 
	// When true, navigation away from the widget is prevented until a new value has been accepted or canceled. 
	bool bControllerInputCaptured;

	// Whether or not we're in fine-tune mode
	bool bIsFineTune;

	// Whether or not the mouse is down
	bool bIsMouseDown;

	/** When true, this slider will be keyboard focusable. Defaults to false. */
	bool bIsFocusable;

private:

	TArray<FSlateBrush*> KnobImages;

	// Resets controller input state. Fires delegates.
	void ResetControllerState();

	// Holds a delegate that is executed when the mouse is pressed and a capture begins.
	FSimpleDelegate OnMouseCaptureBegin;

	// Holds a delegate that is executed when the mouse is let up and a capture ends.
	FSimpleDelegate OnMouseCaptureEnd;

	// Holds a delegate that is executed when capture begins for controller or keyboard.
	FSimpleDelegate OnControllerCaptureBegin;

	// Holds a delegate that is executed when capture ends for controller or keyboard.
	FSimpleDelegate OnControllerCaptureEnd;

	// Holds a delegate that is executed when the slider's value changed.
	FOnFloatValueChanged OnValueChanged;
};
