// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SOverlay.h"

class STextBlock;

// Special tooltip that doesn't follow the mouse position
// Creates a simple window at the designated position.
class SSynthTooltip : public SOverlay
{
public:
	SLATE_BEGIN_ARGS(SSynthTooltip)
	{}
		SLATE_SLOT_ARGUMENT(SOverlay::FOverlaySlot, Slots)
	SLATE_END_ARGS()

	~SSynthTooltip();
	void Construct(const FArguments& InArgs);
	void SetWindowContainerVisibility(bool bShowVisibility);
	void SetOverlayWindowPosition(FVector2D Position);
	void SetOverlayText(const FText& InText);

private:
	TSharedPtr<SWindow> WindowContainer;
	TSharedPtr<STextBlock> TooltipText;
	FVector2D WindowPosition;
	bool bIsVisible;

};
