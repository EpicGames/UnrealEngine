// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UI/Synth2DSliderStyle.h"
#include "Components/Widget.h"
#include "Synth2DSlider.generated.h"

#define UE_API SYNTHESIS_API

class SSynth2DSlider;

DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnMouseCaptureBeginEventSynth2D);
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnMouseCaptureEndEventSynth2D);
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnControllerCaptureBeginEventSynth2D);
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnControllerCaptureEndEventSynth2D);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnFloatValueChangedEventSynth2D, float, Value);

/**
 * A simple widget that shows a sliding bar with a handle that allows you to control the value between 0..1.
 *
 * * No Children
 */
UCLASS(MinimalAPI)
class USynth2DSlider : public UWidget
{
	GENERATED_UCLASS_BODY()

public:
	UPROPERTY(EditAnywhere, Category=Appearance, meta=( ClampMin="0", ClampMax="1", UIMin="0", UIMax="1"))
	float ValueX;

	UPROPERTY(EditAnywhere, Category = Appearance, meta = (ClampMin = "0", ClampMax = "1", UIMin = "0", UIMax = "1"))
	float ValueY;

	/** A bindable delegate to allow logic to drive the value of the widget */
	UPROPERTY()
	FGetFloat ValueXDelegate;

	/** A bindable delegate to allow logic to drive the value of the widget */
	UPROPERTY()
	FGetFloat ValueYDelegate;

public:
	
	/** The progress bar style */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Style", meta=( DisplayName="Style" ))
	FSynth2DSliderStyle WidgetStyle;

	/** The color to draw the slider handle in. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Appearance)
	FLinearColor SliderHandleColor;

	/** Whether the slidable area should be indented to fit the handle. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Appearance, AdvancedDisplay)
	bool IndentHandle;

	/** Whether the handle is interactive or fixed. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Appearance, AdvancedDisplay)
	bool Locked;

	/** The amount to adjust the value by, when using a controller or keyboard */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Appearance, meta=( ClampMin="0", ClampMax="1", UIMin="0", UIMax="1"))
	float StepSize;

	/** Should the slider be focusable? */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Interaction")
	bool IsFocusable;

public:

	/** Invoked when the mouse is pressed and a capture begins. */
	UPROPERTY(BlueprintAssignable, Category="Widget Event")
	FOnMouseCaptureBeginEventSynth2D OnMouseCaptureBegin;

	/** Invoked when the mouse is released and a capture ends. */
	UPROPERTY(BlueprintAssignable, Category="Widget Event")
	FOnMouseCaptureEndEventSynth2D OnMouseCaptureEnd;

	/** Invoked when the controller capture begins. */
	UPROPERTY(BlueprintAssignable, Category = "Widget Event")
	FOnControllerCaptureBeginEventSynth2D OnControllerCaptureBegin;

	/** Invoked when the controller capture ends. */
	UPROPERTY(BlueprintAssignable, Category = "Widget Event")
	FOnControllerCaptureEndEventSynth2D OnControllerCaptureEnd;

	/** Called when the value is changed by slider or typing. */
	UPROPERTY(BlueprintAssignable, Category="Widget Event")
	FOnFloatValueChangedEventSynth2D OnValueChangedX;

	/** Called when the value is changed by slider or typing. */
	UPROPERTY(BlueprintAssignable, Category = "Widget Event")
	FOnFloatValueChangedEventSynth2D OnValueChangedY;

	/** Gets the current value of the slider. */
	UFUNCTION(BlueprintCallable, Category="Behavior")
	UE_API FVector2D GetValue() const;

	/** Sets the current value of the slider. */
	UFUNCTION(BlueprintCallable, Category="Behavior")
	UE_API void SetValue(FVector2D InValue);

	/** Sets if the slidable area should be indented to fit the handle */
	UFUNCTION(BlueprintCallable, Category="Behavior")
	UE_API void SetIndentHandle(bool InValue);

	/** Sets the handle to be interactive or fixed */
	UFUNCTION(BlueprintCallable, Category="Behavior")
	UE_API void SetLocked(bool InValue);

	/** Sets the amount to adjust the value by, when using a controller or keyboard */
	UFUNCTION(BlueprintCallable, Category="Behavior")
	UE_API void SetStepSize(float InValue);

	/** Sets the color of the handle bar */
	UFUNCTION(BlueprintCallable, Category="Appearance")
	UE_API void SetSliderHandleColor(FLinearColor InValue);
	
	// UWidget interface
	UE_API virtual void SynchronizeProperties() override;
	// End of UWidget interface

	// UVisual interface
	UE_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	// End of UVisual interface

#if WITH_EDITOR
	UE_API virtual const FText GetPaletteCategory() override;
#endif

protected:
	/** Native Slate Widget */
	TSharedPtr<SSynth2DSlider> MySlider;

	// UWidget interface
	UE_API virtual TSharedRef<SWidget> RebuildWidget() override;
	// End of UWidget interface

	UE_API void HandleOnValueChangedX(float InValue);
	UE_API void HandleOnValueChangedY(float InValue);
	UE_API void HandleOnMouseCaptureBegin();
	UE_API void HandleOnMouseCaptureEnd();
	UE_API void HandleOnControllerCaptureBegin();
	UE_API void HandleOnControllerCaptureEnd();

protected:
	PROPERTY_BINDING_IMPLEMENTATION(float, ValueX);
	PROPERTY_BINDING_IMPLEMENTATION(float, ValueY);
};

#undef UE_API
