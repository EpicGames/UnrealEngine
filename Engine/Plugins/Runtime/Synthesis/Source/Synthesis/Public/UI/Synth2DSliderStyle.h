// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once



#include "Styling/SlateBrush.h"
#include "Styling/SlateWidgetStyle.h"
#include "Synth2DSliderStyle.generated.h"

#define UE_API SYNTHESIS_API

/**
* Represents the appearance of an SSynth2DSlider
*/
USTRUCT(BlueprintType)
struct FSynth2DSliderStyle : public FSlateWidgetStyle
{
	GENERATED_USTRUCT_BODY()

	UE_API FSynth2DSliderStyle();

	UE_API virtual ~FSynth2DSliderStyle();

	static UE_API void Initialize();

	// Image to use for the 2D handle
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateBrush NormalThumbImage;

		// Image to use for the 2D handle
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateBrush DisabledThumbImage;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateBrush NormalBarImage;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateBrush DisabledBarImage;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateBrush BackgroundImage;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	float BarThickness;

	static UE_API const FName TypeName;
	virtual const FName GetTypeName() const override { return TypeName; };

	UE_API virtual void GetResources(TArray< const FSlateBrush* >& OutBrushes) const override;

	static UE_API const FSynth2DSliderStyle& GetDefault();

};

#undef UE_API
