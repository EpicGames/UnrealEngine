// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UI/SynthKnobStyle.h"
#include "Components/Slider.h"
#include "SynthKnob.generated.h"

#define UE_API SYNTHESIS_API

class SSynthKnob;

/**
 * A simple widget that shows a sliding bar with a handle that allows you to control the value between 0..1.
 *
 * * No Children
 */
UCLASS(MinimalAPI)
class USynthKnob : public UWidget
{
	GENERATED_UCLASS_BODY()

public:
	/** The volume value to display. */
	UPROPERTY(EditAnywhere, Category=Appearance, meta=( ClampMin="0", ClampMax="1", UIMin="0", UIMax="1"))
	float Value;

	/** The amount to adjust the value by, when using a controller or keyboard */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Appearance, meta = (ClampMin = "0", ClampMax = "1", UIMin = "0", UIMax = "1"))
	float StepSize;

	/** The speed of the mouse knob control */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Appearance, meta = (ClampMin = "0", ClampMax = "1", UIMin = "0", UIMax = "1"))
	float MouseSpeed;
	
	/** The speed of the mouse knob control when fine-tuning the knob */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Appearance, meta = (ClampMin = "0", ClampMax = "1", UIMin = "0", UIMax = "1"))
	float MouseFineTuneSpeed;

	/** Enable tool tip window to show parameter information while knob turns */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = SynthTooltip)
	uint32 ShowTooltipInfo:1;

	/** The name of the pararameter. Will show when knob turns. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = SynthTooltip)
	FText ParameterName;

	/** The parameter units (e.g. hz). Will append to synth tooltip info. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = SynthTooltip)
	FText ParameterUnits;

	/** A bindable delegate to allow logic to drive the value of the widget */
	UPROPERTY()
	FGetFloat ValueDelegate;

public:
	
	/** The synth knob style */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Style", meta=( DisplayName="Style" ))
	FSynthKnobStyle WidgetStyle;

	/** Whether the handle is interactive or fixed. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Appearance, AdvancedDisplay)
	bool Locked;

	/** Should the slider be focusable? */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Interaction")
	bool IsFocusable;

public:

	/** Invoked when the mouse is pressed and a capture begins. */
	UPROPERTY(BlueprintAssignable, Category="Widget Event")
	FOnMouseCaptureBeginEvent OnMouseCaptureBegin;

	/** Invoked when the mouse is released and a capture ends. */
	UPROPERTY(BlueprintAssignable, Category="Widget Event")
	FOnMouseCaptureEndEvent OnMouseCaptureEnd;

	/** Invoked when the controller capture begins. */
	UPROPERTY(BlueprintAssignable, Category = "Widget Event")
	FOnControllerCaptureBeginEvent OnControllerCaptureBegin;

	/** Invoked when the controller capture ends. */
	UPROPERTY(BlueprintAssignable, Category = "Widget Event")
	FOnControllerCaptureEndEvent OnControllerCaptureEnd;

	/** Called when the value is changed by slider or typing. */
	UPROPERTY(BlueprintAssignable, Category="Widget Event")
	FOnFloatValueChangedEvent OnValueChanged;

	/** Gets the current value of the slider. */
	UFUNCTION(BlueprintCallable, Category="Behavior")
	UE_API float GetValue() const;

	/** Sets the current value of the slider. */
	UFUNCTION(BlueprintCallable, Category="Behavior")
	UE_API void SetValue(float InValue);

	/** Sets the handle to be interactive or fixed */
	UFUNCTION(BlueprintCallable, Category="Behavior")
	UE_API void SetLocked(bool InValue);

	/** Sets the amount to adjust the value by, when using a controller or keyboard */
	UFUNCTION(BlueprintCallable, Category="Behavior")
	UE_API void SetStepSize(float InValue);

	// UWidget interface
	UE_API virtual void SynchronizeProperties() override;
	// End of UWidget interface

	// UVisual interface
	UE_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	// End of UVisual interface

#if WITH_EDITOR
	UE_API virtual const FText GetPaletteCategory() override;
#endif

protected:
	/** Native Slate Widget */
 	TSharedPtr<SSynthKnob> MySynthKnob;

	// UWidget interface
	UE_API virtual TSharedRef<SWidget> RebuildWidget() override;
	// End of UWidget interface

	UE_API void HandleOnValueChanged(float InValue);
	UE_API void HandleOnMouseCaptureBegin();
	UE_API void HandleOnMouseCaptureEnd();
	UE_API void HandleOnControllerCaptureBegin();
	UE_API void HandleOnControllerCaptureEnd();

protected:
	PROPERTY_BINDING_IMPLEMENTATION(float, Value);
};

#undef UE_API
