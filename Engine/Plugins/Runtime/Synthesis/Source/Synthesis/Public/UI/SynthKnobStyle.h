// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once



#include "Styling/SlateBrush.h"
#include "Styling/SlateWidgetStyle.h"
#include "SynthKnobStyle.generated.h"

#define UE_API SYNTHESIS_API

UENUM(BlueprintType)
enum class ESynthKnobSize : uint8
{
	Medium,
	Large,
	Count UMETA(Hidden)
};

/**
 * Represents the appearance of an SSynthKnob
 */
USTRUCT(BlueprintType)
struct FSynthKnobStyle : public FSlateWidgetStyle
{
	GENERATED_USTRUCT_BODY()

	UE_API FSynthKnobStyle();

	UE_API virtual ~FSynthKnobStyle();

	static UE_API void Initialize();

	static UE_API const FName TypeName;
	virtual const FName GetTypeName() const override { return TypeName; };

	UE_API virtual void GetResources(TArray< const FSlateBrush* >& OutBrushes) const override;

	static UE_API const FSynthKnobStyle& GetDefault();

	// Returns the base brush to use
	UE_API const FSlateBrush* GetBaseBrush() const;

	// Returns the overlay brush to represent the given value
	UE_API const FSlateBrush* GetOverlayBrush() const;

	// Image to use for the large knob
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateBrush LargeKnob;

	// Image to use for the dot handle
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateBrush LargeKnobOverlay;

	// Image to use for the medium large knob
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateBrush MediumKnob;

	// Image to use for the medium knob dot handle
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateBrush MediumKnobOverlay;

	// Image to use for the medium knob dot handle
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	float MinValueAngle;

	// Image to use for the medium knob dot handle
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	float MaxValueAngle;

	/** The size of the knobs to use. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	ESynthKnobSize KnobSize;
	FSynthKnobStyle& SetKnobSize(const ESynthKnobSize& InKnobSize){ KnobSize = InKnobSize; return *this; }

};

#undef UE_API
