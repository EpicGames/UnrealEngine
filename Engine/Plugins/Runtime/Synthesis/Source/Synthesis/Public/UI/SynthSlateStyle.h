// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateWidgetStyle.h"
#include "SynthSlateStyle.generated.h"

#define UE_API SYNTHESIS_API

struct FSlateDynamicImageBrush;

UENUM(BlueprintType)
enum class ESynthSlateSizeType : uint8
{
	Small,
	Medium,
	Large,
	Count UMETA(Hidden)
};

UENUM(BlueprintType)
enum class ESynthSlateColorStyle : uint8
{
	Light,
	Dark,
	Count UMETA(Hidden)
};

struct ISynthSlateResources
{
	ISynthSlateResources();
	virtual ~ISynthSlateResources();

	virtual void LoadResources() = 0;
	virtual const TArray<TSharedPtr<FSlateDynamicImageBrush>>& GetImagesList(ESynthSlateSizeType SizeType, ESynthSlateColorStyle ColorStyle) const = 0;

protected:

	int32 GetNumberForImageName(const FString& ImageName);

	virtual void GetImagesAtPath(const FString& DirPath, TArray<TSharedPtr<FSlateDynamicImageBrush>>& OutImages, const float Size) = 0;
	virtual float GetSize(ESynthSlateSizeType SizeType) const = 0;

	bool bResourcesLoaded;
};


struct FSynthSlateStyleSet
{
	static void Initialize();
	static void Shutdown();

	static TSharedPtr< class FSlateStyleSet > Get();

private:
	static TSharedPtr< class FSlateStyleSet > StyleInstance;
};

/**
 * Represents the appearance of synth UI elements in slate.
 */
USTRUCT(BlueprintType)
struct FSynthSlateStyle : public FSlateWidgetStyle
{
	GENERATED_USTRUCT_BODY()

	UE_API FSynthSlateStyle();

	UE_API virtual ~FSynthSlateStyle();

	static UE_API const FName TypeName;
	virtual const FName GetTypeName() const override { return TypeName; };

	static UE_API const FSynthSlateStyle& GetDefault();

	UE_API const FSlateBrush* GetBrushForValue(const float InValue) const;

	/** The size of the knobs to use. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	ESynthSlateSizeType SizeType;
	FSynthSlateStyle& SetSizeType(const ESynthSlateSizeType& InSizeType){ SizeType = InSizeType; return *this; }

	/** Image to use when the slider bar is in its disabled state */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	ESynthSlateColorStyle ColorStyle;
	FSynthSlateStyle& SetColorStyle(const ESynthSlateColorStyle& InColorStyle){ ColorStyle = InColorStyle; return *this; }

protected:
	virtual ISynthSlateResources* CreateSynthSlateResources() { return nullptr; }

	static UE_API ISynthSlateResources* SynthSlateResources;
};

#undef UE_API
