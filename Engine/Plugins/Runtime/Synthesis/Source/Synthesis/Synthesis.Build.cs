// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class Synthesis : ModuleRules
	{
		public Synthesis(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"Engine",
					"AudioExtensions",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"AudioMixer",
					"SignalProcessing",
					"UMG",
					"Slate",
					"SlateCore",
					"InputCore",
					"Projects",
				}
			);

			if (Target.Type == TargetType.Editor)
			{
				PrivateDependencyModuleNames.Add("AudioSynesthesiaCore");
			}
		}
	}
}
