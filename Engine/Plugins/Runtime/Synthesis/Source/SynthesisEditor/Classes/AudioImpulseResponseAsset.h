// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "AssetTypeActions_Base.h"
#include "Sound/SampleBufferIO.h"
#include "AudioImpulseResponseAsset.generated.h"


class FAssetTypeActions_AudioImpulseResponse : public FAssetTypeActions_Base
{
public:
	virtual FText GetName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_AudioImpulseResponse", "Audio Impulse Response"); }
	virtual FColor GetTypeColor() const override { return FColor(0, 255, 0); }
	virtual UClass* GetSupportedClass() const override;
	virtual uint32 GetCategories() override { return EAssetTypeCategories::Sounds; }
};

class FAudioImpulseResponseExtension
{
public:
	static void RegisterMenus();
	static void ExecuteCreateImpulseResponse(const struct FToolMenuContext& MenuContext);
};


UCLASS(MinimalAPI, hidecategories = Object)
class UAudioImpulseResponseFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	//~ Begin UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	//~ Begin UFactory Interface
	
	TWeakObjectPtr<USoundWave> StagedSoundWave;

private:
	Audio::FSoundWavePCMLoader Loader;
};
