// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "AssetTypeActions_Base.h"
#include "EpicSynth1PresetBank.generated.h"

class FAssetTypeActions_ModularSynthPresetBank : public FAssetTypeActions_Base
{
public:
	virtual FText GetName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_EpicSynth1PresetBank", "Modular Synth Preset Bank"); }
	virtual FColor GetTypeColor() const override { return FColor(175, 100, 155); }
	virtual const TArray<FText>& GetSubMenus() const override;
	virtual UClass* GetSupportedClass() const override;
	virtual uint32 GetCategories() override { return EAssetTypeCategories::Sounds; }
};

UCLASS(MinimalAPI, hidecategories = Object)
class UModularSynthPresetBankFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	//~ Begin UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	//~ Begin UFactory Interface	
};
