// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "AssetTypeActions_Base.h"
#include "MonoWaveTablePresetBank.generated.h"

class FAssetTypeActions_MonoWaveTableSynthPreset : public FAssetTypeActions_Base
{
public:
	virtual FText GetName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_MonoWaveTableSynthPreset", "Mono Wave Table Synth Preset Bank"); }
	virtual FColor GetTypeColor() const override { return FColor(0, 255, 0); }
	virtual UClass* GetSupportedClass() const override;
	virtual const TArray<FText>& GetSubMenus() const override;
	virtual uint32 GetCategories() override { return EAssetTypeCategories::Sounds; }
};

UCLASS(MinimalAPI, hidecategories = Object)
class UMonoWaveTableSynthPresetFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	//~ Begin UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	//~ Begin UFactory Interface	
};
