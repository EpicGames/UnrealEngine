// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Engine/DeveloperSettings.h"
#include "SynthesisEditorSettings.generated.h"


UCLASS(config = EditorSettings, defaultconfig, meta = (DisplayName = "Synthesis and DSP Plugin"))
class USynthesisEditorSettings : public UDeveloperSettings
{
	GENERATED_BODY()
};
