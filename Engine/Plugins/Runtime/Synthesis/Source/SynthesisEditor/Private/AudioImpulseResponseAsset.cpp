// Copyright Epic Games, Inc. All Rights Reserved.
#include "AudioImpulseResponseAsset.h"

#include "Algo/AnyOf.h"
#include "AssetTypeActions/AssetDefinition_SoundWave.h"
#include "ContentBrowserMenuContexts.h"
#include "DSP/FloatArrayMath.h"
#include "EffectConvolutionReverb.h"
#include "Misc/PackageName.h"
#include "Sound/SoundWaveProcedural.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "ToolMenuSection.h"
#include "AssetDefinitionRegistry.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioImpulseResponseAsset)

#define LOCTEXT_NAMESPACE "AssetTypeActions"

UClass* FAssetTypeActions_AudioImpulseResponse::GetSupportedClass() const
{
	return UAudioImpulseResponse::StaticClass();
}

void FAudioImpulseResponseExtension::RegisterMenus()
{
	if (!UToolMenus::IsToolMenuUIEnabled())
	{
		return;
	}

	const UAssetDefinition* AssetDefinition = UAssetDefinitionRegistry::Get()->GetAssetDefinitionForClass(USoundWave::StaticClass());
	FToolMenuSection* Section = CastChecked<UAssetDefinition_SoundAssetBase>(AssetDefinition)->FindSoundContextMenuSection("Sound");
	check(Section);

	Section->AddDynamicEntry("SoundWaveAssetConversion_CreateImpulseResponse", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
	{
		if (const UContentBrowserAssetContextMenuContext* Context = InSection.FindContext<UContentBrowserAssetContextMenuContext>())
		{
			if (!Algo::AnyOf(Context->SelectedAssets, [](const FAssetData& AssetData){ return AssetData.IsInstanceOf<USoundWaveProcedural>(); }))
			{
				const TAttribute<FText> Label = LOCTEXT("SoundWave_CreateImpulseResponse", "Create Impulse Response");
				const TAttribute<FText> ToolTip = LOCTEXT("SoundWave_CreateImpulseResponseTooltip", "Creates an impulse response asset using the selected sound wave.");
				const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.ImpulseResponse");
				const FToolMenuExecuteAction UIAction = FToolMenuExecuteAction::CreateStatic(&FAudioImpulseResponseExtension::ExecuteCreateImpulseResponse);

				InSection.AddMenuEntry("SoundWave_CreateImpulseResponse", Label, ToolTip, Icon, UIAction);
			}
		}
	}));
}

void FAudioImpulseResponseExtension::ExecuteCreateImpulseResponse(const FToolMenuContext& MenuContext)
{
	const FString DefaultSuffix = TEXT("_IR");
	FAssetToolsModule& AssetToolsModule = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools");

	// Create the factory used to generate the asset
	UAudioImpulseResponseFactory* Factory = NewObject<UAudioImpulseResponseFactory>();
	
	// only converts 0th selected object for now (see return statement)
	if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(MenuContext))
	{
		for (USoundWave* Wave : Context->LoadSelectedObjectsIf<USoundWave>([](const FAssetData& AssetData) { return !AssetData.IsInstanceOf<USoundWaveProcedural>(); }))
		{
			Factory->StagedSoundWave = Wave; // WeakPtr gets reset by the Factory after it is consumed

			// Determine an appropriate name
			FString Name;
			FString PackagePath;
			AssetToolsModule.Get().CreateUniqueAssetName(Wave->GetOutermost()->GetName(), DefaultSuffix, PackagePath, Name);

			// create new asset
			AssetToolsModule.Get().CreateAsset(Name, FPackageName::GetLongPackagePath(PackagePath), UAudioImpulseResponse::StaticClass(), Factory);
		}
	}
}

UAudioImpulseResponseFactory::UAudioImpulseResponseFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UAudioImpulseResponse::StaticClass();

	bCreateNew = false;
	bEditorImport = false;
	bEditAfterNew = true;
}

UObject* UAudioImpulseResponseFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UAudioImpulseResponse* NewAsset = NewObject<UAudioImpulseResponse>(InParent, InName, Flags);

	if (StagedSoundWave.IsValid())
	{
		USoundWave* Wave = StagedSoundWave.Get();

		TArray<uint8> ImportedSoundWaveData;
		uint32 ImportedSampleRate;
		uint16 ImportedChannelCount;
		Wave->GetImportedSoundWaveData(ImportedSoundWaveData, ImportedSampleRate, ImportedChannelCount);

		NewAsset->NumChannels = ImportedChannelCount;
		NewAsset->SampleRate = ImportedSampleRate;

		NewAsset->ImpulseResponse.Reset();

		int32 NumSamples = ImportedSoundWaveData.Num() / sizeof(int16);
		if (NumSamples > 0)
		{
			NewAsset->ImpulseResponse.AddUninitialized(NumSamples);

			// Convert to float.
			const int16* InputBuffer = (int16*)ImportedSoundWaveData.GetData();
			float* OutputBuffer = NewAsset->ImpulseResponse.GetData();

			Audio::ArrayPcm16ToFloat(MakeArrayView(InputBuffer, NumSamples), MakeArrayView(OutputBuffer, NumSamples));
		}

#if WITH_EDITORONLY_DATA
		NewAsset->bIsEvenChannelCount = NewAsset->NumChannels % 2 == 0;
#endif

		StagedSoundWave.Reset();
	}

	return NewAsset;
}

#undef LOCTEXT_NAMESPACE

