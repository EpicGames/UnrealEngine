// Copyright Epic Games, Inc. All Rights Reserved.
#include "EpicSynth1PresetBank.h"

#include "AudioAnalytics.h"
#include "SynthComponents/EpicSynth1Component.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EpicSynth1PresetBank)


#define LOCTEXT_NAMESPACE "AssetTypeActions"

UClass* FAssetTypeActions_ModularSynthPresetBank::GetSupportedClass() const
{
	return UModularSynthPresetBank::StaticClass();
}

const TArray<FText>& FAssetTypeActions_ModularSynthPresetBank::GetSubMenus() const
{
	static const TArray<FText> SubMenus
	{
		FText(LOCTEXT("AssetSoundSynthesisSubMenu", "Legacy"))
	};

	return SubMenus;
}

UModularSynthPresetBankFactory::UModularSynthPresetBankFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UModularSynthPresetBank::StaticClass();

	bCreateNew = true;
	bEditorImport = false;
	bEditAfterNew = true;
}

UObject* UModularSynthPresetBankFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UModularSynthPresetBank* NewPresetBank = NewObject<UModularSynthPresetBank>(InParent, InName, Flags);
	Audio::Analytics::RecordEvent_Usage(TEXT("SynthesisAndDSPEffects.ModularSynthPresetBankCreated"));
	return NewPresetBank;
}

#undef LOCTEXT_NAMESPACE
