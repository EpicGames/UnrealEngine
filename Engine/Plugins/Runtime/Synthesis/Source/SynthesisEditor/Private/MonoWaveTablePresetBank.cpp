// Copyright Epic Games, Inc. All Rights Reserved.
#include "MonoWaveTablePresetBank.h"

#include "AudioAnalytics.h"
#include "SynthComponents/SynthComponentMonoWaveTable.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MonoWaveTablePresetBank)

#define LOCTEXT_NAMESPACE "AssetTypeActions"


UClass* FAssetTypeActions_MonoWaveTableSynthPreset::GetSupportedClass() const
{
	return UMonoWaveTableSynthPreset::StaticClass();
}

const TArray<FText>& FAssetTypeActions_MonoWaveTableSynthPreset::GetSubMenus() const
{
	static const TArray<FText> SubMenus
	{
		FText(LOCTEXT("AssetSoundSynthesisSubMenu", "Legacy"))
	};

	return SubMenus;
}

UMonoWaveTableSynthPresetFactory::UMonoWaveTableSynthPresetFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UMonoWaveTableSynthPreset::StaticClass();

	bCreateNew = true;
	bEditorImport = false;
	bEditAfterNew = true;
}

UObject* UMonoWaveTableSynthPresetFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UMonoWaveTableSynthPreset* NewPresetBank = NewObject<UMonoWaveTableSynthPreset>(InParent, InName, Flags);
	Audio::Analytics::RecordEvent_Usage(TEXT("SynthesisAndDSPEffects.MonoWaveTableSynthPresetCreated"));
	return NewPresetBank;
}
#undef LOCTEXT_NAMESPACE
