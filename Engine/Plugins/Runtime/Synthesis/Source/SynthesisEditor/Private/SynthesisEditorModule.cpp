// Copyright Epic Games, Inc. All Rights Reserved.

#include "SynthesisEditorModule.h"
#include "AudioEditorModule.h"
#include "EpicSynth1PresetBank.h"
#include "MonoWaveTablePresetBank.h"
#include "AudioImpulseResponseAsset.h"
#include "ToolMenus.h"


DEFINE_LOG_CATEGORY(LogSynthesisEditor);

IMPLEMENT_MODULE(FSynthesisEditorModule, SynthesisEditor)


void FSynthesisEditorModule::StartupModule()
{
	LLM_SCOPE(ELLMTag::AudioSynthesis);
	IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();

	AssetTools.RegisterAssetTypeActions(MakeShared<FAssetTypeActions_ModularSynthPresetBank>());
	AssetTools.RegisterAssetTypeActions(MakeShared<FAssetTypeActions_MonoWaveTableSynthPreset>());
	AssetTools.RegisterAssetTypeActions(MakeShared<FAssetTypeActions_AudioImpulseResponse>());

	// Now that we've loaded this module, we need to register our effect preset actions
	IAudioEditorModule* AudioEditorModule = &FModuleManager::LoadModuleChecked<IAudioEditorModule>("AudioEditor");
	AudioEditorModule->RegisterEffectPresetAssetActions();

	UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateRaw(this, &FSynthesisEditorModule::RegisterMenus));
}

void FSynthesisEditorModule::ShutdownModule()
{
	UToolMenus::UnRegisterStartupCallback(this);
	UToolMenus::UnregisterOwner(this);
	UToolMenus::UnregisterOwner(UE_MODULE_NAME);
}

void FSynthesisEditorModule::RegisterMenus()
{
	LLM_SCOPE(ELLMTag::AudioSynthesis);
	FToolMenuOwnerScoped MenuOwner(this);
	FAudioImpulseResponseExtension::RegisterMenus();
}
