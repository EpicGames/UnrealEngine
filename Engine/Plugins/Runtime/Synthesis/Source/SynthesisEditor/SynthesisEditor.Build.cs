// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class SynthesisEditor : ModuleRules
	{
		public SynthesisEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange
			(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"EditorFramework",
					"UnrealEd",
					"AudioEditor",
					"Synthesis",
					"AudioMixer",
					"ToolMenus",
					"Slate",
					"SlateCore",
					"ContentBrowser",
				}
			);

			PrivateDependencyModuleNames.AddRange
			(
				new string[]
				{
					"AssetDefinition",
					"AudioSynesthesiaCore",
					"DeveloperSettings",
					"UMG",
					"UMGEditor",
					"SignalProcessing"
				}
			);

			PrivateIncludePathModuleNames.AddRange
			(
				new string[]
				{
					"AssetTools"
				}
			);
		}
	}
}
