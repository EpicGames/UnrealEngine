{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.1",
	"FriendlyName" : "Synthesis and DSP Effects",
	"Description" : "A variety of realtime synthesizers and DSP source and submix effects.",
	"Category" : "Audio",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : true,
	"CanContainContent" : true,
	"IsBetaVersion" : false,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "Synthesis",
			"Type" : "Runtime",
			"LoadingPhase" : "PreDefault"
		},
		{
			"Name" : "SynthesisEditor",
			"Type" : "Editor",
			"LoadingPhase" : "Default"
		}
	],
	"Plugins": [
		{
			"Name": "AudioSynesthesia",
			"Enabled": true
		}
	]
}